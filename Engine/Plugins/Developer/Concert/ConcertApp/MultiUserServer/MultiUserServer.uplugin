{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Multi User Server",
	"Description": "Visualizes the multi-user server",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealMultiUserSlateServer" ],
	"Modules": [
        {
          "Name": "MultiUserServer",
          "Type": "UncookedOnly",
          "LoadingPhase": "Default",
          "ProgramAllowList": [ "UnrealMultiUserSlateServer" ]
        }
	],
	"Plugins": [
		{
			"Name": "ConcertMain",
			"Enabled": true
		},
        {
          "Name": "ConcertSharedSlate",
          "Enabled": true
        },
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncServer",
			"Enabled": true
		}
	]
}
