// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MultiUserServer : ModuleRules
	{
		public MultiUserServer(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"Concert",
					"ConcertServer",
					"ConcertSharedSlate",
					"ConcertSyncCore",
					"ConcertTransport",
					"EngineSettings",
					"InputCore",
					"Messaging",
					"OutputLog",
					"Projects",
					"Slate",
					"SlateCore",
					"StandaloneRenderer",	
					"ToolWidgets",
					"ToolMenus",
					"WorkspaceMenuStructure" // Needed for OutputLog module to work correctly
				}
			);
			
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"ConcertSyncServer",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					"ConcertSyncServer"
				});
		}
	}
}
