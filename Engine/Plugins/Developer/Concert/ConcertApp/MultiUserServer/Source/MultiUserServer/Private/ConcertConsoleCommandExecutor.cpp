// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertConsoleCommandExecutor.h"

#include "ConsoleSettings.h"
#include "Framework/Commands/InputChord.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"

namespace UE::MultiUserServer
{
	FName FConcertConsoleCommandExecutor::StaticName()
	{
		static const FName CmdExecName = TEXT("Cmd");
		return CmdExecName;
	}

	FName FConcertConsoleCommandExecutor::GetName() const
	{
		return StaticName();
	}

	FText FConcertConsoleCommandExecutor::GetDisplayName() const
	{
		return LOCTEXT("ConsoleCommandExecutorDisplayName", "Cmd");
	}

	FText FConcertConsoleCommandExecutor::GetDescription() const
	{
		return LOCTEXT("ConsoleCommandExecutorDescription", "Execute Unreal Console Commands");
	}

	FText FConcertConsoleCommandExecutor::GetHintText() const
	{
		return LOCTEXT("ConsoleCommandExecutorHintText", "Enter Console Command");
	}

	void FConcertConsoleCommandExecutor::GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out)
	{
		auto OnConsoleVariable = [&Out](const TCHAR *Name, IConsoleObject* CVar)
		{
			if (CVar->IsEnabled())
			{
				Out.Add(FConsoleSuggestion(Name, CVar->GetDetailedHelp().ToString()));
			}
		};

		IConsoleManager& ConsoleManager = IConsoleManager::Get();
		ConsoleManager.ForEachConsoleObjectThatContains(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), Input);
		for (const FString& Name : GetDefault<UConsoleSettings>()->GetFilteredManualAutoCompleteCommands(Input))
		{
			FString HelpString;
			// Try to find a console object for this entry in order to retrieve a help string if possible :
			const TCHAR* NamePtr = *Name;
			if (IConsoleObject* CObj = ConsoleManager.FindConsoleObject(*FParse::Token(NamePtr, /*UseEscape = */false), /*bTrackFrequentCalls = */false); CObj && CObj->IsEnabled())
			{
				HelpString = CObj->GetDetailedHelp().ToString();
			}
			Out.Add(FConsoleSuggestion(Name, HelpString));
		}
	}

	void FConcertConsoleCommandExecutor::GetExecHistory(TArray<FString>& Out)
	{
		IConsoleManager::Get().GetConsoleHistory(TEXT(""), Out);
	}

	bool FConcertConsoleCommandExecutor::Exec(const TCHAR* Input)
	{
		IConsoleManager::Get().AddConsoleHistoryEntry(TEXT(""), Input);
	
		FOutputDevice& Output = *GLog;
		UWorld* World = nullptr;
	
		bool bSuccess = IConsoleManager::Get().ProcessUserConsoleInput(Input, Output, World);
		if (!bSuccess)
		{
			bSuccess |= FSelfRegisteringExec::StaticExec(World, Input, Output);
		}

		return bSuccess;
	}

	bool FConcertConsoleCommandExecutor::AllowHotKeyClose() const
	{
		return true;
	}

	bool FConcertConsoleCommandExecutor::AllowMultiLine() const
	{
		return false;
	}

	FInputChord FConcertConsoleCommandExecutor::GetHotKey() const
	{
		return FInputChord();
	}

	FInputChord FConcertConsoleCommandExecutor::GetIterateExecutorHotKey() const
	{
		return FInputChord();
	}
}

#undef LOCTEXT_NAMESPACE
