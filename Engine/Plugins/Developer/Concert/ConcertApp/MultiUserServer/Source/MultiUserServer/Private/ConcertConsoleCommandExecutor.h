// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

struct FInputChord;

namespace UE::MultiUserServer
{
	/** Executes commands put into the output log */
	class FConcertConsoleCommandExecutor : public IConsoleCommandExecutor
	{
	public:
	
		static FName StaticName();

		//~ Begin IConsoleCommandExecutor Interface
		virtual FName GetName() const override;
		virtual FText GetDisplayName() const override;
		virtual FText GetDescription() const override;
		virtual FText GetHintText() const override;
		virtual void GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out) override;
		virtual void GetExecHistory(TArray<FString>& Out) override;
		virtual bool Exec(const TCHAR* Input) override;
		virtual bool AllowHotKeyClose() const override;
		virtual bool AllowMultiLine() const override;
		virtual FInputChord GetHotKey() const override;
		virtual FInputChord GetIterateExecutorHotKey() const override;
		//~ End IConsoleCommandExecutor Interface
	};
}
