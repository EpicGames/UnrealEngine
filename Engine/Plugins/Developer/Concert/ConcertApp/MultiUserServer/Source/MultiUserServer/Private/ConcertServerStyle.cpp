// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/StyleColors.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"

TSharedPtr<FSlateStyleSet> FConcertServerStyle::StyleInstance = nullptr;

void FConcertServerStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FConcertServerStyle::Shutdown()
{
	if (StyleInstance)
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
		ensure(StyleInstance.IsUnique());
		StyleInstance.Reset();
	}
}

const ISlateStyle& FConcertServerStyle::Get()
{
	return *StyleInstance;
}

FName FConcertServerStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("ConcertServerStyle"));
	return StyleSetName;
}

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FConcertServerStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( FConcertServerStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( FConcertServerStyle::InContent(RelativePath, ".svg"), __VA_ARGS__)

FString FConcertServerStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MultiUserServer"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedRef<FSlateStyleSet> FConcertServerStyle::Create()
{
	TSharedRef<FSlateStyleSet> StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
	StyleSet->SetParentStyleName("CoreStyle");
	FAppStyle::SetAppStyleSet(*StyleSet); // Makes the app icons appear
	
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate/Starship/Insights"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	
	const FVector2D Icon16x16(16.0f, 16.0f); 
	const FVector2D Icon24x24(24.0f, 24.0f); 
	const FVector2D Icon32x32(32.0f, 32.0f); 
	const FVector2D Icon48x48(48.0f, 48.0f); 
	
	// Application icons
	StyleSet->Set("AppIcon", new IMAGE_PLUGIN_BRUSH("Icons/App/MultiUserApp_48", Icon48x48));
	StyleSet->Set("AppIconPadding", FMargin(5.0f, 5.0f, 5.0f, 5.0f));
	StyleSet->Set("AppIcon.Small", new IMAGE_PLUGIN_BRUSH("Icons/App/MultiUserApp_24", Icon24x24));
	StyleSet->Set("AppIconPadding.Small", FMargin(4.0f, 4.0f, 0.0f, 0.0f));
	
	// Icons
	StyleSet->Set("Concert.MultiUser", new IMAGE_PLUGIN_BRUSH("Icons/icon_MultiUser_32x", Icon32x32));
	StyleSet->Set("Concert.SessionContent.ColumnHeader", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Package_16x", Icon16x16));
	// Icons: Package
	StyleSet->Set("Concert.SessionContent.PackageAdded", new IMAGE_PLUGIN_BRUSH_SVG("Icons/PackageAdded_16x", Icon16x16));
	StyleSet->Set("Concert.SessionContent.PackageDeleted", new IMAGE_PLUGIN_BRUSH_SVG("Icons/PackageDeleted_16x", Icon16x16));
	StyleSet->Set("Concert.SessionContent.PackageRenamed", new IMAGE_PLUGIN_BRUSH_SVG("Icons/PackageRenamed_16x", Icon16x16));
	StyleSet->Set("Concert.SessionContent.PackageSaved", new IMAGE_PLUGIN_BRUSH_SVG("Icons/PackageSaved_16x", Icon16x16));
	// Icons: Ack
	StyleSet->Set("Concert.Ack.Ack", new IMAGE_PLUGIN_BRUSH("Icons/Ack_Ack_16x", Icon16x16));
	StyleSet->Set("Concert.Ack.Success", new IMAGE_PLUGIN_BRUSH("Icons/Ack_Success_16x", Icon16x16));
	StyleSet->Set("Concert.Ack.Failure", new IMAGE_PLUGIN_BRUSH("Icons/Ack_Fail_16x", Icon16x16));
	// Icons: Package transmission
	StyleSet->Set("Concert.PackageTransmission.Success", new IMAGE_PLUGIN_BRUSH("Icons/Ack_Success_16x", Icon16x16));
	StyleSet->Set("Concert.PackageTransmission.Failure", new IMAGE_PLUGIN_BRUSH("Icons/Ack_Fail_16x", Icon16x16));
	// Icons: Muting
	StyleSet->Set("Concert.Muted",     new IMAGE_PLUGIN_BRUSH("Icons/Muted_16x", Icon16x16));
	StyleSet->Set("Concert.Unmuted",     new IMAGE_PLUGIN_BRUSH("Icons/Unmuted_16x", Icon16x16));
	// Icons: DependencyView
	StyleSet->Set("Concert.HighlightHardDependencies",     new IMAGE_PLUGIN_BRUSH("Icons/Ack_Ack_16x", Icon16x16));

	// New icons
	StyleSet->Set("Concert.Icon.Archive", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Archive_16", Icon16x16));
	StyleSet->Set("Concert.Icon.Client", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Client_16", Icon16x16));
	StyleSet->Set("Concert.Icon.CreateMultiUser", new IMAGE_PLUGIN_BRUSH_SVG("Icons/CreateMultiUser_16", Icon16x16));
	StyleSet->Set("Concert.Icon.Export", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Export_16", Icon16x16));
	StyleSet->Set("Concert.Icon.Import", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Import_16", Icon16x16));
	StyleSet->Set("Concert.Icon.LogServer", new IMAGE_PLUGIN_BRUSH_SVG("Icons/LogServer", Icon16x16));
	StyleSet->Set("Concert.Icon.LogSession", new IMAGE_PLUGIN_BRUSH_SVG("Icons/LogSession", Icon16x16));
	StyleSet->Set("Concert.Icon.MultiUser", new IMAGE_PLUGIN_BRUSH_SVG("Icons/MultiUser_16", Icon16x16));
	StyleSet->Set("Concert.Icon.Package", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Package", Icon16x16));
	StyleSet->Set("Concert.Icon.Server", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Server_16", Icon16x16));

	// Clients tab
	StyleSet->Set("Concert.Clients.DropShadow", new IMAGE_PLUGIN_BOX_BRUSH("ClientThumbnailDropShadow", FMargin(4.0f / 64.0f)));
	StyleSet->Set("Concert.Clients.ThumbnailAreaHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f));
	StyleSet->Set("Concert.Clients.ThumbnailAreaBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f));
	StyleSet->Set("Concert.Clients.ThumbnailTitle", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f));
	StyleSet->Set("Concert.Clients.ThumbnailCurveBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 0.0f));
	StyleSet->Set("Concert.Clients.ThumbnailFooter", new FSlateRoundedBoxBrush(FStyleColors::Panel, 0.0f));
	StyleSet->Set("Concert.Clients.TileTableRow", FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateNoResource() )
		.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(FSlateNoResource())
		.SetActiveHoveredBrush(FSlateNoResource())
		.SetInactiveBrush(FSlateNoResource())
		.SetInactiveHoveredBrush(FSlateNoResource())
		.SetTextColor(FSlateColor())
		.SetSelectedTextColor(FSlateColor())
		);
	StyleSet->Set("Concert.Clients.ClientNameTileFont", DEFAULT_FONT("Regular", 16));
	
	// Graph colours
	// Blueish
	StyleSet->Set("Concert.Clients.NetworkGraph.Sent.LineColor", 0.7f * FLinearColor(.24f, .38f, .76f, 0.f));
	StyleSet->Set("Concert.Clients.NetworkGraph.Sent.FillColor",  0.4f * FLinearColor(.24f, .38f, .76f, 1.f));
	// Reddish
	StyleSet->Set("Concert.Clients.NetworkGraph.Received.LineColor", 0.7f * FLinearColor(.85f, .27f, .27f, 0.f));
	StyleSet->Set("Concert.Clients.NetworkGraph.Received.FillColor",  0.4f * FLinearColor(.85f, .27f, .27f, 1.f));

	constexpr float LineBrightness = 0.35f;
	constexpr float TextBrightness = 0.5f;
	StyleSet->Set("Concert.Clients.NetworkGraph.GraphSeparatorLine.Thickness", 3.f);
	StyleSet->Set("Concert.Clients.NetworkGraph.GraphSeparatorLine.LineColor", FLinearColor(LineBrightness, LineBrightness, LineBrightness, 0.35f));
	StyleSet->Set("Concert.Clients.NetworkGraph.HorizontalHelperLine.Thickness", 2.f);
	StyleSet->Set("Concert.Clients.NetworkGraph.HorizontalHelperLine.LineColor", FLinearColor(LineBrightness, LineBrightness, LineBrightness, 0.35f));
	StyleSet->Set("Concert.Clients.NetworkGraph.HorizontalHelperLine.TextColor", FLinearColor(TextBrightness, TextBrightness, TextBrightness));
	
	return StyleSet;
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_PLUGIN_BOX_BRUSH
#undef IMAGE_PLUGIN_BRUSH_SVG

