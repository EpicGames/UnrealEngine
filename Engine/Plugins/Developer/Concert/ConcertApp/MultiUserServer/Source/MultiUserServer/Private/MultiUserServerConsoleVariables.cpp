// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserServerConsoleVariables.h"


TAutoConsoleVariable<bool> UE::MultiUserServer::ConsoleVariables::CVarLogActivityDependencyGraphOnDelete(TEXT("MultiUserServer.LogActivityDependencyGraphOnDelete"), false, TEXT("When a user requests to delete any activity, log the dependency graph (in GraphViz format)"));
