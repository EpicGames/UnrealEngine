// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

namespace UE::MultiUserServer::ConsoleVariables
{
	/** When a user requests to delete any activity, log the dependency graph (in GraphViz format) */
	extern TAutoConsoleVariable<bool> CVarLogActivityDependencyGraphOnDelete;
}
