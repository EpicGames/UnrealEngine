// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserServerModule.h"

#include "ConcertConsoleCommandExecutor.h"
#include "ConcertFrontendStyle.h"
#include "ConcertServerStyle.h"
#include "ConcertSyncServerLoopInitArgs.h"
#include "Window/ModalWindowManager.h"
#include "Window/ConcertServerWindowController.h"

#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "StandaloneRenderer.h"
#include "Misc/ConfigCacheIni.h"

#include "Dialog/DialogCommands.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"

namespace UE::MultiUserServer
{
    void FConcertServerUIModule::StartupModule() 
	{
		MultiUserServerLayoutIni = GConfig->GetConfigFilename(TEXT("MultiUserServerLayout"));

		if (!FDialogCommands::IsRegistered())
		{
			FDialogCommands::Register();
		}
	}
	
	void FConcertServerUIModule::ShutdownModule()
	{
		FConcertServerStyle::Shutdown();

    	ModalWindowManager.Reset();
		WindowController.Reset();
		FSlateApplication::Shutdown();
	}
	
	void FConcertServerUIModule::InitSlateForServer(FConcertSyncServerLoopInitArgs& InitArgs)
	{
		if (ensureMsgf(!WindowController.IsValid(), TEXT("InitSlateForServer is designed to be called at most once to create UI to run alongside the Multi User server.")))
		{
			InitArgs.PostInitServerLoop.AddRaw(this, &FConcertServerUIModule::InitializeSlateApplication);
			InitArgs.TickPostGameThread.AddRaw(this, &FConcertServerUIModule::TickSlate);
		}
	}
	
	void FConcertServerUIModule::PreInitializeMultiUser()
    {
    	FModuleManager::Get().LoadModuleChecked("SlateCore");
    	FConcertServerStyle::Initialize();
    	FConcertFrontendStyle::Initialize();

    	// Log history must be initialized before MU server loop init prints any messages	
    	FModuleManager::Get().LoadModuleChecked("OutputLog");
    }
    
    void FConcertServerUIModule::InitializeSlateApplication(TSharedRef<IConcertSyncServer> SyncServer)
    {
    	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());
    	const FText ApplicationTitle = LOCTEXT("AppTitle", "Unreal Multi User Server");
    	FGlobalTabmanager::Get()->SetApplicationTitle(ApplicationTitle);

		PreInitializeMultiUser();

    	CommandExecutor = MakeUnique<FConcertConsoleCommandExecutor>();
    	IModularFeatures::Get().RegisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), CommandExecutor.Get());

        WindowController = MakeShared<FConcertServerWindowController>(FConcertServerWindowInitParams{ SyncServer, MultiUserServerLayoutIni });
    	const TSharedRef<SWindow> MainWindow = WindowController->CreateWindow();
    	
    	ModalWindowManager = MakeShared<FModalWindowManager>(MainWindow);
    }
    
    void FConcertServerUIModule::TickSlate(double Tick)
    {
    	FSlateApplication::Get().PumpMessages();
    	FSlateApplication::Get().Tick();
    }
}

IMPLEMENT_MODULE(UE::MultiUserServer::FConcertServerUIModule, MultiUserServer);

#undef LOCTEXT_NAMESPACE
