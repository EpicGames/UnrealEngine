// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "IMultiUserServerModule.h"


class IConcertSyncServer;
class IModuleInterface;

namespace UE::MultiUserServer
{
	class FModalWindowManager;
	class FConcertConsoleCommandExecutor;
	class FConcertServerWindowController;
	
	class FConcertServerUIModule : public IMultiUserServerModule
    {
    public:
    
		static inline FConcertServerUIModule& Get()
    	{
    		static const FName ModuleName = "MultiUserServer";
    		return FModuleManager::LoadModuleChecked<FConcertServerUIModule>(ModuleName);
    	}

		TSharedPtr<FModalWindowManager> GetModalWindowManager() const { return ModalWindowManager; }
            
        //~ Begin IModuleInterface Interface
        virtual void StartupModule() override ;
        virtual void ShutdownModule() override;
        //~ End IModuleInterface Interface
        
        //~ Begin IMultiUserServerModule Interface
        virtual void InitSlateForServer(FConcertSyncServerLoopInitArgs& InitArgs) override;
        //~ End IMultiUserServerModule Interface
        
    private:
    
        /** Config path storing layout config. */
        FString MultiUserServerLayoutIni;
    
        /** Handles execution of commands */
        TUniquePtr<FConcertConsoleCommandExecutor> CommandExecutor;
        
		/** Manages fake modal windows */
		TSharedPtr<FModalWindowManager> ModalWindowManager;
		
        /** Creates and manages window. Only one instance per application. */
        TSharedPtr<FConcertServerWindowController> WindowController;
    
        void PreInitializeMultiUser();
        void InitializeSlateApplication(TSharedRef<IConcertSyncServer> SyncServer);
        
        void TickSlate(double Tick);
    };
}
