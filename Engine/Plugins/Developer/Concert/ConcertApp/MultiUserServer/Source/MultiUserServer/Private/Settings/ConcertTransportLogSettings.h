// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendUtils.h"
#include "ConcertTransportLogSettings.generated.h"

/**
 * 
 */
UCLASS(config = MultiUserServerUserSettings, DefaultConfig)
class MULTIUSERSERVER_API UConcertTransportLogSettings : public UObject
{
	GENERATED_BODY()
public:

	static UConcertTransportLogSettings* GetSettings();

	/** How to displayed FConcertLog::Timestamp */
	UPROPERTY(Config)
	ETimeFormat TimestampTimeFormat = ETimeFormat::Absolute;
};
