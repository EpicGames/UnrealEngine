// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserServerColumnVisibilitySettings.h"

#include "Misc/CoreDelegates.h"

namespace UE::MultiUserServer
{
	static bool bIsShutdown = false;
}

UMultiUserServerColumnVisibilitySettings::UMultiUserServerColumnVisibilitySettings()
{
	OnSessionBrowserColumnVisibilityChanged().AddLambda([this](const FColumnVisibilitySnapshot&)
	{
		SaveConfig();
	});
	OnDeleteActivityDialogColumnVisibility().AddLambda([this](const FColumnVisibilitySnapshot&)
	{
		SaveConfig();
	});
	OnArchivedActivityBrowserColumnVisibility().AddLambda([this](const FColumnVisibilitySnapshot&)
	{
		SaveConfig();
	});
	OnLiveActivityBrowserColumnVisibility().AddLambda([this](const FColumnVisibilitySnapshot&)
	{
		SaveConfig();
	});
	OnLiveSessionContentColumnVisibility().AddLambda([this](const FColumnVisibilitySnapshot&)
	{
		SaveConfig();
	});
	OnTransportLogColumnVisibility().AddLambda([this](const FColumnVisibilitySnapshot&)
	{
		SaveConfig();
	});
	OnOnPackageTransmissionColumnVisibilityChanged().AddLambda([this](const FColumnVisibilitySnapshot&)
	{
		SaveConfig();
	});

	FCoreDelegates::OnPreExit.AddLambda([]()
	{
		UE::MultiUserServer::bIsShutdown = true;
	});
}

UMultiUserServerColumnVisibilitySettings* UMultiUserServerColumnVisibilitySettings::GetSettings()
{
	// If we're shutting down GetMutableDefault will return garbage - this function may be called by destructors when this module is unloaded
	return UE::MultiUserServer::bIsShutdown
		? nullptr
		: GetMutableDefault<UMultiUserServerColumnVisibilitySettings>();
}
