// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertHeaderRowUtils.h"
#include "MultiUserServerColumnVisibilitySettings.generated.h"

/**
 * 
 */
UCLASS(Config = MultiUserServerUserSettings, DefaultConfig)
class MULTIUSERSERVER_API UMultiUserServerColumnVisibilitySettings : public UObject
{
	GENERATED_BODY()
public:

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnColumnVisibilitySnapshotChanged, const FColumnVisibilitySnapshot& /*NewValue*/);

	UMultiUserServerColumnVisibilitySettings();
	
	static UMultiUserServerColumnVisibilitySettings* GetSettings();

	const FColumnVisibilitySnapshot& GetSessionBrowserColumnVisibility() const { return SessionBrowserColumnVisibility; }
	void SetSessionBrowserColumnVisibility(FColumnVisibilitySnapshot NewValue) { SessionBrowserColumnVisibility = MoveTemp(NewValue); OnSessionBrowserColumnVisibilityChangedEvent.Broadcast(SessionBrowserColumnVisibility); }
	FOnColumnVisibilitySnapshotChanged& OnSessionBrowserColumnVisibilityChanged() { return OnSessionBrowserColumnVisibilityChangedEvent; }

	const FColumnVisibilitySnapshot& GetDeleteActivityDialogColumnVisibility() const { return DeleteActivityDialogColumnVisibility; }
	void SetDeleteActivityDialogColumnVisibility(FColumnVisibilitySnapshot NewValue) { DeleteActivityDialogColumnVisibility = MoveTemp(NewValue); OnDeleteActivityDialogColumnVisibilityEvent.Broadcast(DeleteActivityDialogColumnVisibility); }
	FOnColumnVisibilitySnapshotChanged& OnDeleteActivityDialogColumnVisibility() { return OnDeleteActivityDialogColumnVisibilityEvent; }
	
	const FColumnVisibilitySnapshot& GetArchivedActivityBrowserColumnVisibility() const { return ArchivedActivityBrowserColumnVisibility; }
	void SetArchivedActivityBrowserColumnVisibility(FColumnVisibilitySnapshot NewValue) { ArchivedActivityBrowserColumnVisibility = MoveTemp(NewValue); OnArchivedActivityBrowserColumnVisibilityEvent.Broadcast(ArchivedActivityBrowserColumnVisibility); }
	FOnColumnVisibilitySnapshotChanged& OnArchivedActivityBrowserColumnVisibility() { return OnArchivedActivityBrowserColumnVisibilityEvent; }

	const FColumnVisibilitySnapshot& GetLiveActivityBrowserColumnVisibility() const { return LiveActivityBrowserColumnVisibility; }
	void SetLiveActivityBrowserColumnVisibility(FColumnVisibilitySnapshot NewValue) { LiveActivityBrowserColumnVisibility = MoveTemp(NewValue); OnLiveActivityBrowserColumnVisibilityEvent.Broadcast(LiveActivityBrowserColumnVisibility); }
	FOnColumnVisibilitySnapshotChanged& OnLiveActivityBrowserColumnVisibility() { return OnLiveActivityBrowserColumnVisibilityEvent; }

	const FColumnVisibilitySnapshot& GetLiveSessionContentColumnVisibility() const { return LiveSessionContentColumnVisibility; }
	void SetLiveSessionContentColumnVisibility(FColumnVisibilitySnapshot NewValue) { LiveSessionContentColumnVisibility = MoveTemp(NewValue); OnLiveSessionContentColumnVisibilityEvent.Broadcast(LiveSessionContentColumnVisibility); }
	FOnColumnVisibilitySnapshotChanged& OnLiveSessionContentColumnVisibility() { return OnLiveSessionContentColumnVisibilityEvent; }

	const FColumnVisibilitySnapshot& GetTransportLogColumnVisibility() const { return TransportLogColumnVisibility; }
	void SetTransportLogColumnVisibility(FColumnVisibilitySnapshot NewValue) { TransportLogColumnVisibility = MoveTemp(NewValue); OnTransportLogColumnVisibilityEvent.Broadcast(TransportLogColumnVisibility); }
	FOnColumnVisibilitySnapshotChanged& OnTransportLogColumnVisibility() { return OnTransportLogColumnVisibilityEvent; }

	const FColumnVisibilitySnapshot& GetPackageTransmissionColumnVisibility() const { return PackageTransmissionColumnVisibility; }
	void SetPackageTransmissionColumnVisibility(FColumnVisibilitySnapshot NewValue) { PackageTransmissionColumnVisibility = MoveTemp(NewValue); OnPackageTransmissionColumnVisibilityChangedEvent.Broadcast(PackageTransmissionColumnVisibility); }
	FOnColumnVisibilitySnapshotChanged& OnOnPackageTransmissionColumnVisibilityChanged() { return OnPackageTransmissionColumnVisibilityChangedEvent; }
	
private:
	
	UPROPERTY(Config)
	FColumnVisibilitySnapshot SessionBrowserColumnVisibility;
	FOnColumnVisibilitySnapshotChanged OnSessionBrowserColumnVisibilityChangedEvent;

	UPROPERTY(Config)
	FColumnVisibilitySnapshot DeleteActivityDialogColumnVisibility;
	FOnColumnVisibilitySnapshotChanged OnDeleteActivityDialogColumnVisibilityEvent;

	UPROPERTY(Config)
	FColumnVisibilitySnapshot ArchivedActivityBrowserColumnVisibility;
	FOnColumnVisibilitySnapshotChanged OnArchivedActivityBrowserColumnVisibilityEvent;

	UPROPERTY(Config)
	FColumnVisibilitySnapshot LiveActivityBrowserColumnVisibility;
	FOnColumnVisibilitySnapshotChanged OnLiveActivityBrowserColumnVisibilityEvent;

	UPROPERTY(Config)
	FColumnVisibilitySnapshot LiveSessionContentColumnVisibility;
	FOnColumnVisibilitySnapshotChanged OnLiveSessionContentColumnVisibilityEvent;

	UPROPERTY(Config)
	FColumnVisibilitySnapshot TransportLogColumnVisibility;
	FOnColumnVisibilitySnapshotChanged OnTransportLogColumnVisibilityEvent;

	UPROPERTY(Config)
	FColumnVisibilitySnapshot PackageTransmissionColumnVisibility;
	FOnColumnVisibilitySnapshotChanged OnPackageTransmissionColumnVisibilityChangedEvent;
};
