// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserServerPackageTransmissionSettings.h"

UMultiUserServerPackageTransmissionSettings* UMultiUserServerPackageTransmissionSettings::GetSettings()
{
	return GetMutableDefault<UMultiUserServerPackageTransmissionSettings>();
}