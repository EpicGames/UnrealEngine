// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendUtils.h"
#include "MultiUserServerPackageTransmissionSettings.generated.h"

/**
 * 
 */
UCLASS(config = MultiUserServerUserSettings, DefaultConfig)
class MULTIUSERSERVER_API UMultiUserServerPackageTransmissionSettings : public UObject
{
	GENERATED_BODY()
public:

	static UMultiUserServerPackageTransmissionSettings* GetSettings();

	/** How to display FPackageTransmissionEntry::Time */
	UPROPERTY(Config)
	ETimeFormat TimestampTimeFormat = ETimeFormat::Absolute;
};
