// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserServerUserPreferences.h"

UMultiUserServerUserPreferences* UMultiUserServerUserPreferences::GetSettings()
{
	return GetMutableDefault<UMultiUserServerUserPreferences>();
}
