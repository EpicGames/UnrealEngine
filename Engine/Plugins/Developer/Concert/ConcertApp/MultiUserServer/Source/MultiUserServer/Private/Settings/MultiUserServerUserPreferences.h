// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MultiUserServerUserPreferences.generated.h"

/**
 * 
 */
UCLASS(Config = MultiUserServerUserSettings, DefaultConfig)
class MULTIUSERSERVER_API UMultiUserServerUserPreferences : public UObject
{
	GENERATED_BODY()
public:
	
	static UMultiUserServerUserPreferences* GetSettings();

	UPROPERTY(Config)
	bool bWarnUserAboutMuting = true;

	UPROPERTY(Config)
	bool bWarnUserAboutUnmuting = true;
};
