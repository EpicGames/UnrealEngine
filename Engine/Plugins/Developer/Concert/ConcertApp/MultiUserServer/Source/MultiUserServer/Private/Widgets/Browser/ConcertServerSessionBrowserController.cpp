// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerSessionBrowserController.h"

#include "ConcertServerEvents.h"
#include "ConcertServerStyle.h"
#include "IConcertModule.h"
#include "IConcertServer.h"
#include "IConcertSession.h"
#include "IConcertSyncServer.h"
#include "SConcertServerSessionBrowser.h"
#include "Session/Browser/Items/ConcertSessionTreeItem.h"

#include "Misc/AsyncTaskNotification.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SConcertTabViewBase.h"
#include "Window/ConcertServerTabs.h"
#include "Window/ConcertServerWindowController.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FConcertServerSessionBrowserController"

namespace UE::MultiUserServer
{
	int32 FConcertServerSessionBrowserController::GetNumConnectedClients(const FGuid& SessionId) const
	{
		TSharedPtr<IConcertServerSession> Session = ServerInstance->GetConcertServer()->GetLiveSession(SessionId);
		return ensure(Session) ? Session->GetSessionClients().Num() : 0;
	}

	FConcertServerSessionBrowserController::~FConcertServerSessionBrowserController()
	{
		ConcertServerEvents::OnLiveSessionCreated().RemoveAll(this);
		ConcertServerEvents::OnLiveSessionDestroyed().RemoveAll(this);
		ConcertServerEvents::OnArchivedSessionCreated().RemoveAll(this);
		ConcertServerEvents::OnArchivedSessionDestroyed().RemoveAll(this);
	}

	void FConcertServerSessionBrowserController::Init(const FConcertComponentInitParams& Params)
	{
		ServerInstance = Params.Server;
		Owner = Params.WindowController;
		
		FGlobalTabmanager::Get()->RegisterTabSpawner(
				ConcertServerTabs::GetSessionBrowserTabId(),
				FOnSpawnTab::CreateRaw(this, &FConcertServerSessionBrowserController::SpawnSessionBrowserTab, Params.WindowController->GetRootWindow())
			)
			.SetDisplayName(LOCTEXT("SessionBrowserTabTitleLong", "Session Browser"))
			.SetTooltipText(LOCTEXT("SessionBrowserTooltipText", "A section to browse, start, archive, and restore server sessions."))
			.SetIcon(FSlateIcon(FConcertServerStyle::GetStyleSetName(), TEXT("Concert.Icon.MultiUser"))
		);
		Params.MainStack->AddTab(ConcertServerTabs::GetSessionBrowserTabId(), ETabState::OpenedTab)
			->SetForegroundTab(ConcertServerTabs::GetSessionBrowserTabId());

		ConcertServerEvents::OnLiveSessionCreated().AddSP(this, &FConcertServerSessionBrowserController::OnLiveSessionCreated);
		ConcertServerEvents::OnLiveSessionDestroyed().AddSP(this, &FConcertServerSessionBrowserController::OnLiveSessionDestroyed);
		ConcertServerEvents::OnArchivedSessionCreated().AddSP(this, &FConcertServerSessionBrowserController::OnArchivedSessionCreated);
		ConcertServerEvents::OnArchivedSessionDestroyed().AddSP(this, &FConcertServerSessionBrowserController::OnArchivedSessionDestroyed);
	}

	void FConcertServerSessionBrowserController::OpenTab()
	{
		FGlobalTabmanager::Get()->TryInvokeTab(ConcertServerTabs::GetSessionBrowserTabId());
	}

	TArray<FConcertServerInfo> FConcertServerSessionBrowserController::GetServers() const
	{
		return { ServerInstance->GetConcertServer()->GetServerInfo() };
	}

	TArray<IConcertSessionBrowserController::FActiveSessionInfo> FConcertServerSessionBrowserController::GetActiveSessions() const
	{
		const FConcertServerInfo& ServerInfo = ServerInstance->GetConcertServer()->GetServerInfo();
		const TArray<TSharedPtr<IConcertServerSession>> ServerSessions = ServerInstance->GetConcertServer()->GetLiveSessions();
		
		TArray<FActiveSessionInfo> Result;
		Result.Reserve(ServerSessions.Num());
		for (const TSharedPtr<IConcertServerSession>& LiveSession : ServerSessions)
		{
			FActiveSessionInfo Info{ ServerInfo, LiveSession->GetSessionInfo(), LiveSession->GetSessionClients() };
			Result.Add(Info);
		}

		return Result;
	}

	TArray<IConcertSessionBrowserController::FArchivedSessionInfo> FConcertServerSessionBrowserController::GetArchivedSessions() const
	{
		const FConcertServerInfo& ServerInfo = ServerInstance->GetConcertServer()->GetServerInfo();
		const TArray<FConcertSessionInfo> ConcertSessionInfos = ServerInstance->GetConcertServer()->GetArchivedSessionInfos();
		
		TArray<FArchivedSessionInfo> Result;
		Result.Reserve(ConcertSessionInfos.Num());
		for (const FConcertSessionInfo& SessionInfo : ConcertSessionInfos)
		{
			Result.Emplace(ServerInfo, SessionInfo);
		}

		return Result;
	}

	TOptional<FConcertSessionInfo> FConcertServerSessionBrowserController::GetActiveSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const
	{
		const TSharedPtr<IConcertServerSession> LiveSession = ServerInstance->GetConcertServer()->GetLiveSession(SessionId);
		return LiveSession ? LiveSession->GetSessionInfo() : TOptional<FConcertSessionInfo>{};
	}

	TOptional<FConcertSessionInfo> FConcertServerSessionBrowserController::GetArchivedSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const
	{
		return ServerInstance->GetConcertServer()->GetArchivedSessionInfo(SessionId);
	}

	void FConcertServerSessionBrowserController::CreateSession(const FGuid& ServerAdminEndpointId, const FString& SessionName, const FString& ProjectName)
	{
		FConcertSessionInfo SessionInfo = ServerInstance->GetConcertServer()->CreateSessionInfo();
		SessionInfo.SessionName = SessionName;
		SessionInfo.Settings.Initialize();
		if (!ProjectName.IsEmpty())
		{
			SessionInfo.Settings.ProjectName = ProjectName;
		}
		FConcertSessionVersionInfo VersionInfo;
		VersionInfo.Initialize(false /* bSupportMixedBuildTypes */ );
		SessionInfo.VersionInfos.Emplace(VersionInfo);

		FAsyncTaskNotification Notification(
			MakeAsyncNotification(FText::Format(LOCTEXT("CreateSessionFmt.InProgress", "Created Session '{0}'"), FText::FromString(SessionName)))
			);
		FText FailureReason = FText::GetEmpty();
		const bool bSuccess = ServerInstance->GetConcertServer()->CreateSession(SessionInfo, FailureReason) != nullptr;

		CloseAsyncNotification(
			Notification,
			bSuccess,
			bSuccess
				? FText::Format(LOCTEXT("CreateSessionFmt.Success", "Created Session '{0}'"), FText::FromString(SessionName))
				: FText::Format(LOCTEXT("CreateSessionFmt.Failure", "Failed to create Session '{0}'"), FText::FromString(SessionName)),
			FailureReason
			);
	}

	void FConcertServerSessionBrowserController::ArchiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& ArchiveName, const FConcertSessionFilter& SessionFilter)
	{
		FAsyncTaskNotification Notification(
			MakeAsyncNotification(FText::Format(LOCTEXT("ArchivedSessionFmt.InProgress", "Archiving new Session '{0}'"), FText::FromString(ArchiveName)))
			);
		
		FText FailureReason = FText::GetEmpty();
		const bool bSuccess = ServerInstance->GetConcertServer()->ArchiveSession(SessionId, ArchiveName, SessionFilter, FailureReason).IsValid();
		
		CloseAsyncNotification(
			Notification,
			bSuccess,
			bSuccess
				? FText::Format(LOCTEXT("ArchiveSessionFmt.Success", "Archived Session '{0}'"), FText::FromString(ArchiveName))
				: FText::Format(LOCTEXT("ArchiveSessionFmt.Failure", "Failed to archive Session '{0}'"), FText::FromString(ArchiveName)),
			FailureReason
			);
	}

	void FConcertServerSessionBrowserController::RestoreSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& RestoredName, const FConcertSessionFilter& SessionFilter)
	{
		if (TOptional<FConcertSessionInfo> SessionInfo = GetArchivedSessionInfo(ServerAdminEndpointId, SessionId))
		{
			FAsyncTaskNotification Notification(
				MakeAsyncNotification(FText::Format(LOCTEXT("RestoreSessionFmt.InProgress", "Restoring Session '{0}'"), FText::FromString(RestoredName)))
				);
			
			FText FailureReason = FText::GetEmpty();
			SessionInfo->SessionName = RestoredName;
			const bool bSuccess = ServerInstance->GetConcertServer()->RestoreSession(SessionId, *SessionInfo, SessionFilter, FailureReason).IsValid();
			
			CloseAsyncNotification(
				Notification,
				bSuccess,
				bSuccess
					? FText::Format(LOCTEXT("RestoreSessionFmt.Success", "Restored Session '{0}'"),  FText::FromString(RestoredName))
					: FText::Format(LOCTEXT("RestoreSessionFmt.Failure", "Failed to restore Session '{0}'"), FText::FromString(RestoredName)),
				FailureReason
			);
		}
	}

	void FConcertServerSessionBrowserController::DeleteSessions(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& SessionIds)
	{
		// We want a special message to be displayed when only deleting single items
		if (SessionIds.Num() == 1)
		{
			const FString SessionName = GetSessionName(ServerAdminEndpointId, SessionIds[0]).Get(SessionIds[0].ToString());
			FAsyncTaskNotification Notification(
				MakeAsyncNotification(FText::Format(LOCTEXT("DeleteSessionFmt.InProgress", "Deleting session '{0}'"), FText::FromString(SessionName)))
				);
			
			FText FailureReason;
			const bool bSuccess = DeleteSession(ServerAdminEndpointId, SessionIds[0], FailureReason);
			
			CloseAsyncNotification(
				Notification,
				bSuccess,
				bSuccess
					? FText::Format(LOCTEXT("DeleteSessionFmt.Success", "Deleted Session '{0}'"), FText::FromString(SessionName))
					: FText::Format(LOCTEXT("DeleteSessionFmt.Failure", "Failed to archive Session '{0}'"), FText::FromString(SessionIds[0].ToString())),
				FailureReason
			);
			return;
		}

		FAsyncTaskNotification Notification(
			MakeAsyncNotification(FText::Format(LOCTEXT("DeleteSessionsFmt.InProgress", "Deleting {0} sessions"), SessionIds.Num()))
			);
		
		using FSessionName = FString;
		using FReasonText = FText;
		
		TArray<TPair<FSessionName, FReasonText>> Failures;
		int32 NumberSuccessful = 0;
		for (const FGuid& SessionId : SessionIds)
		{
			FText FailureReason;
			if (DeleteSession(ServerAdminEndpointId, SessionId, FailureReason))
			{
				++NumberSuccessful;
			}
			else 
			{
				const TOptional<FString> SessionName = GetSessionName(ServerAdminEndpointId, SessionId);
				Failures.Add({ SessionName.Get(SessionId.ToString()), FailureReason });
			}
		}
		
		if (Failures.Num() == 0)
		{
			CloseAsyncNotification(
				Notification,
				true,
				FText::Format(LOCTEXT("DeleteSessionsFmt.Success", "Deleted {0} Sessions"), NumberSuccessful)
			);
		}
		else
		{
			// Instead of a notification for each failure, show one big one so the user does not have to manually close each (cuz that's annoying...)
			TStringBuilder<512> SubText;
			for (const TPair<FSessionName, FReasonText>& Failure : Failures)
			{
				SubText << Failure.Key << TEXT(": ") << Failure.Value.ToString() << TEXT("\n");
			}
			CloseAsyncNotification(
				Notification,
				false,
				FText::Format(LOCTEXT("DeletedSessionsFmt.Failure", "Deleted {0} of {1} Sessions."), NumberSuccessful, SessionIds.Num()),
				FText::FromString(SubText.ToString())
			);
		}
	}

	TSharedRef<SDockTab> FConcertServerSessionBrowserController::SpawnSessionBrowserTab(const FSpawnTabArgs& Args, TSharedPtr<SWindow> RootWindow)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label(LOCTEXT("SessionBrowserTabTitle", "Sessions"))
			.TabRole(MajorTab)
			.CanEverClose(false);

		DockTab->SetContent(
			SAssignNew(ConcertBrowser, SConcertServerSessionBrowser, SharedThis(this))
				.ConstructUnderMajorTab(DockTab)
				.ConstructUnderWindow(RootWindow)
				.DoubleClickLiveSession(this, &FConcertServerSessionBrowserController::OpenSession)
				.DoubleClickArchivedSession(this, &FConcertServerSessionBrowserController::OpenSession)
			);

		return DockTab;
	}

	void FConcertServerSessionBrowserController::RefreshSessionList()
	{
		if (ConcertBrowser.IsValid())
		{
			// 1. Accumulate several events in same tick so list is only updated once
			// 2. Several events, e.g. OnLiveSessionCreated, are called before the session creation is visible to getter functions
			ConcertBrowser->RequestRefreshListNextTick();
		}
	}

	void FConcertServerSessionBrowserController::OpenSession(const TSharedPtr<FConcertSessionTreeItem>& SessionItem)
	{
		Owner.Pin()->OpenSessionTab(SessionItem->SessionId);
	}

	void FConcertServerSessionBrowserController::RenameActiveSessionInternal(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName)
	{
		if (const TOptional<FConcertSessionInfo> SessionInfo = GetActiveSessionInfo(ServerAdminEndpointId, SessionId))
		{
			RenameSessionInternal(SessionId, NewName, *SessionInfo);
		}
	}

	void FConcertServerSessionBrowserController::RenameArchivedSessionInternal(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName)
	{
		if (const TOptional<FConcertSessionInfo> SessionInfo = GetArchivedSessionInfo(ServerAdminEndpointId, SessionId))
		{
			RenameSessionInternal(SessionId, NewName, *SessionInfo);
		}
	}

	void FConcertServerSessionBrowserController::RenameSessionInternal(const FGuid& SessionId, const FString& NewName, const FConcertSessionInfo& SessionInfo)
	{
		FAsyncTaskNotification Notification(
			MakeAsyncNotification(FText::Format(LOCTEXT("RenameSessionFmt.InProgress", "Renaming Session {0} "), FText::FromString(SessionInfo.SessionName)))
			);
		
		FText FailureReason = FText::GetEmpty();
		const bool bSuccess = ServerInstance->GetConcertServer()->RenameSession(SessionId, NewName, FailureReason);

		CloseAsyncNotification(
				Notification,
				bSuccess,
				bSuccess
					? FText::Format(LOCTEXT("RenameSessionFmt.Success", "Rename Session '{0}' as '{1}'"), FText::FromString(SessionInfo.SessionName), FText::FromString(NewName))
					: FText::Format(LOCTEXT("RenameSessionFmt.Failure", "Failed to rename Session '{0}' as '{1}'"), FText::FromString(SessionInfo.SessionName), FText::FromString(NewName)),
				FailureReason
			);
	}

	bool FConcertServerSessionBrowserController::DeleteSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, FText& FailureReason)
	{
		const TOptional<FConcertSessionInfo> ArchivedInfo = GetArchivedSessionInfo(ServerAdminEndpointId, SessionId);
		const TOptional<FConcertSessionInfo> LiveInfo = GetActiveSessionInfo(ServerAdminEndpointId, SessionId);
		if (const TOptional<FString> SessionName = GetSessionName(ServerAdminEndpointId, SessionId))
		{
			return ServerInstance->GetConcertServer()->DestroySession(SessionId, FailureReason);
		}
		return false;
	}

	TOptional<FString> FConcertServerSessionBrowserController::GetSessionName(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const
	{
		const TOptional<FConcertSessionInfo> ArchivedInfo = GetArchivedSessionInfo(ServerAdminEndpointId, SessionId);
		const TOptional<FConcertSessionInfo> LiveInfo = GetActiveSessionInfo(ServerAdminEndpointId, SessionId);
		const TOptional<FString> SessionName = ArchivedInfo ? ArchivedInfo->SessionName : LiveInfo ? LiveInfo->SessionName : TOptional<FString>{};
		return SessionName;
	}

	FAsyncTaskNotificationConfig FConcertServerSessionBrowserController::MakeAsyncNotification(const FText& Title, const FText& Details)
	{
		FAsyncTaskNotificationConfig NotificationConfig;
		NotificationConfig.bIsHeadless = false;
		NotificationConfig.bKeepOpenOnFailure = true;
		NotificationConfig.TitleText = Title;
		return NotificationConfig;
	}

	void FConcertServerSessionBrowserController::CloseAsyncNotification(FAsyncTaskNotification& Notification, bool bSuccess, const FText& Title, const FText& Details)
	{
		Notification.SetComplete(Title, Details, bSuccess);

		if (bSuccess)
		{
			ConcertBrowser->RequestRefreshListNextTick();
		}
	}
}

#undef LOCTEXT_NAMESPACE 
