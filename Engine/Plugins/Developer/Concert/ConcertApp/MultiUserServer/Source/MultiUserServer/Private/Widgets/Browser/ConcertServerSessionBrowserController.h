// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Session/Browser/IConcertSessionBrowserController.h"
#include "Window/IConcertComponent.h"

class FAsyncTaskNotification;
struct FAsyncTaskNotificationConfig;
struct FConcertServerInfo;
struct FConcertSessionFilter;
struct FConcertSessionInfo;

class IConcertServerSession;
class IConcertServer;
class FConcertSessionTreeItem;
class FSpawnTabArgs;
class SDockTab;

namespace UE::MultiUserServer
{
	class SConcertServerSessionBrowser;
	
	/** Implements controller in model-view-controller pattern. */
	class FConcertServerSessionBrowserController
		:
		public TSharedFromThis<FConcertServerSessionBrowserController>,
		public IConcertComponent,
		public IConcertSessionBrowserController
	{
	public:

		int32 GetNumConnectedClients(const FGuid& SessionId) const;

		~FConcertServerSessionBrowserController();

		//~ Begin IConcertComponent Interface
		virtual void Init(const FConcertComponentInitParams& Params) override;
		//~ End IConcertComponent Interface

		void OpenTab();

		//~ Begin IConcertSessionBrowserController Interface
		virtual TArray<FConcertServerInfo> GetServers() const override;
		virtual TArray<FActiveSessionInfo> GetActiveSessions() const override;
		virtual TArray<FArchivedSessionInfo> GetArchivedSessions() const override;
		virtual TOptional<FConcertSessionInfo> GetActiveSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const override;
		virtual TOptional<FConcertSessionInfo> GetArchivedSessionInfo(const FGuid& AdminEndpoint, const FGuid& SessionId) const override;

		virtual void CreateSession(const FGuid& ServerAdminEndpointId, const FString& SessionName, const FString& ProjectName) override;
		virtual void ArchiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& ArchiveName, const FConcertSessionFilter& SessionFilter) override;
		virtual void RestoreSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& RestoredName, const FConcertSessionFilter& SessionFilter) override;
		virtual void RenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) override { RenameActiveSessionInternal(ServerAdminEndpointId, SessionId, NewName); }
		virtual void RenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) override { RenameArchivedSessionInternal(ServerAdminEndpointId, SessionId, NewName); }
		virtual void DeleteSessions(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& SessionIds) override;

		// The server operator always has permission for these actions:
		virtual bool CanRenameActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override { return true; }
		virtual bool CanRenameArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override { return true; }
		virtual bool CanDeleteActiveSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override { return true; }
		virtual bool CanDeleteArchivedSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override { return true; }
		virtual bool CanEverCreateSessions() const override { return false; }
		//~ Begin IConcertSessionBrowserController Interface

	private:

		/** Server instance we're representing */
		TSharedPtr<IConcertSyncServer> ServerInstance;
		/** Used to open selected sessions */
		TWeakPtr<FConcertServerWindowController> Owner;

		TSharedPtr<SConcertServerSessionBrowser> ConcertBrowser;

		TSharedRef<SDockTab> SpawnSessionBrowserTab(const FSpawnTabArgs& Args, TSharedPtr<SWindow> RootWindow);

		// Update view when session list changes
		void OnLiveSessionCreated(bool, const IConcertServer&, TSharedRef<IConcertServerSession>) { RefreshSessionList(); }
		void OnLiveSessionDestroyed(const IConcertServer&, TSharedRef<IConcertServerSession>) { RefreshSessionList(); }
		void OnArchivedSessionCreated(bool, const IConcertServer&, const FString&, const FConcertSessionInfo&) { RefreshSessionList(); }
		void OnArchivedSessionDestroyed(const IConcertServer&, const FGuid&) { RefreshSessionList(); }

		void RefreshSessionList();

		// Session actions
		void OpenSession(const TSharedPtr<FConcertSessionTreeItem>& SessionItem);
		void RenameActiveSessionInternal(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName);
		void RenameArchivedSessionInternal(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName);
		void RenameSessionInternal(const FGuid& SessionId, const FString& NewName, const FConcertSessionInfo& SessionInfo);
		bool DeleteSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, FText& FailureReason);

		TOptional<FString> GetSessionName(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const;

		FAsyncTaskNotificationConfig MakeAsyncNotification(const FText& Title, const FText& Details = FText::GetEmpty());
		void CloseAsyncNotification(FAsyncTaskNotification& Notification, bool bSuccess, const FText& Title, const FText& Details = FText::GetEmpty());
	};
}

