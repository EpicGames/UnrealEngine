// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertServerSessionBrowser.h"

#include "ConcertServerStyle.h"
#include "Dialog/SCustomDialog.h"
#include "MultiUserServerModule.h"
#include "Session/Browser/ConcertBrowserUtils.h"
#include "Settings/MultiUserServerColumnVisibilitySettings.h"
#include "Widgets/Browser/ConcertServerSessionBrowserController.h"
#include "Window/ConcertServerTabs.h"
#include "Window/ModalWindowManager.h"

#include "Dialog/SMessageDialog.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertServerSessionBrowser"

namespace UE::MultiUserServer
{
	const FName SConcertServerSessionBrowser::SessionBrowserTabId("SessionBrowserTabId");

	void SConcertServerSessionBrowser::Construct(const FArguments& InArgs, TSharedRef<FConcertServerSessionBrowserController> InController)
	{
		Controller = InController;
		SConcertTabViewWithManagerBase::Construct(
			SConcertTabViewWithManagerBase::FArguments()
			.ConstructUnderWindow(InArgs._ConstructUnderWindow)
			.ConstructUnderMajorTab(InArgs._ConstructUnderMajorTab)
			.CreateTabs(FCreateTabs::CreateLambda([this, &InArgs](const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				CreateTabs(InTabManager, InLayout, InArgs);
			}))
			.LayoutName("ConcertSessionBrowserTabView_v0.1"),
			ConcertServerTabs::GetSessionBrowserTabId()
			);
	}

	void SConcertServerSessionBrowser::CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FArguments& InArgs)
	{
		const TSharedRef<FWorkspaceItem> WorkspaceItem = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("SessionBrowser", "Session Browser"));
		
		InTabManager->RegisterTabSpawner(SessionBrowserTabId, FOnSpawnTab::CreateSP(this, &SConcertServerSessionBrowser::SpawnSessionBrowserTab, InArgs._DoubleClickLiveSession, InArgs._DoubleClickArchivedSession))
			.SetDisplayName(LOCTEXT("SessionBrowserTabLabel", "Sessions"))
			.SetGroup(WorkspaceItem)
			.SetIcon(FSlateIcon(FConcertServerStyle::GetStyleSetName(), TEXT("Concert.Icon.MultiUser")));

		InLayout->AddArea
		(
			FTabManager::NewPrimaryArea()->Split
			(
				FTabManager::NewStack()
				->AddTab(SessionBrowserTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
		);
	}

	TSharedRef<SDockTab> SConcertServerSessionBrowser::SpawnSessionBrowserTab(const FSpawnTabArgs& InTabArgs, FSessionDelegate DoubleClickLiveSession, FSessionDelegate DoubleClickArchivedSession)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("SessionBrowserTabLabel", "Sessions"))
			.TabRole(PanelTab)
			[
				MakeSessionTableView(MoveTemp(DoubleClickLiveSession), MoveTemp(DoubleClickArchivedSession))
			]; 
	}

	TSharedRef<SWidget> SConcertServerSessionBrowser::MakeSessionTableView(FSessionDelegate DoubleClickLiveSession, FSessionDelegate DoubleClickArchivedSession)
	{
		SearchText = MakeShared<FText>();
		return SAssignNew(SessionBrowser, SConcertSessionBrowser, Controller.Pin().ToSharedRef(), SearchText)
			.OnLiveSessionDoubleClicked(MoveTemp(DoubleClickLiveSession))
			.OnArchivedSessionDoubleClicked(MoveTemp(DoubleClickArchivedSession))
			.PostRequestedDeleteSession(this, &SConcertServerSessionBrowser::RequestDeleteSession)
			// Pretend a modal dialog said no - RequestDeleteSession will show non-modal dialog
			.AskUserToDeleteSessions_Lambda([](auto) { return false; })
			.ColumnVisibilitySnapshot(UMultiUserServerColumnVisibilitySettings::GetSettings()->GetSessionBrowserColumnVisibility())
			.SaveColumnVisibilitySnapshot_Lambda([](const FColumnVisibilitySnapshot& Snapshot)
			{
				UMultiUserServerColumnVisibilitySettings::GetSettings()->SetSessionBrowserColumnVisibility(Snapshot);
			});
	}

	void SConcertServerSessionBrowser::RequestDeleteSession(const TArray<TSharedPtr<FConcertSessionTreeItem>>& SessionItems) 
	{
		if (SessionItems.Num() == 0)
		{
			return;
		}
		
		const FText Message = [this, &SessionItems]()
		{
			if (SessionItems.Num() > 1)
			{
				return FText::Format(
				LOCTEXT("DeletedMultipleDescription", "Deleting a session will force all connected clients to disconnect and all associated data to be removed.\n\nDelete {0} sessions?"),
					SessionItems.Num()
				);
			}
				
			switch (SessionItems[0]->Type)
			{
			case FConcertSessionTreeItem::EType::ActiveSession:
				return FText::Format(
					LOCTEXT("DeletedActiveDescription", "There {0}|plural(one=is,other=are) {0} connected {0}|plural(one=client,other=clients) in the current session.\nDeleting a session will force all connected clients to disconnect.\n\nDelete {1}?"),
					Controller.Pin()->GetNumConnectedClients(SessionItems[0]->SessionId),
					FText::FromString(SessionItems[0]->SessionName)
				);
			case FConcertSessionTreeItem::EType::ArchivedSession:
				return FText::Format(
					LOCTEXT("DeleteArchivedDescription", "Deleting a session will cause all associated data to be removed.\n\nDelete {0}?"),
					FText::FromString(SessionItems[0]->SessionName)
					);
			default:
				checkNoEntry();
				return FText::GetEmpty();
			}
		}();
		DeleteSessionsWithFakeModalQuestion(Message, SessionItems);
	}

	void SConcertServerSessionBrowser::DeleteSessionsWithFakeModalQuestion(const FText& Message, const TArray<TSharedPtr<FConcertSessionTreeItem>>& SessionItems)
	{
		auto DeleteArchived = [WeakController = TWeakPtr<FConcertServerSessionBrowserController>(Controller), SessionItems]()
		{
			if (const TSharedPtr<FConcertServerSessionBrowserController> PinnedController = WeakController.Pin())
			{
				ConcertBrowserUtils::RequestItemDeletion(*PinnedController.Get(), SessionItems);
			}
		};
		const TSharedRef<SMessageDialog> Dialog = SNew(SMessageDialog)
			.Title(LOCTEXT("DisconnectUsersTitle", "Delete session?"))
			.Icon(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
			.Message(Message)
			.UseScrollBox(false)
			.Buttons({
				SMessageDialog::FButton(LOCTEXT("DeleteArchivedButton", "Delete"))
					.SetOnClicked(FSimpleDelegate::CreateLambda(DeleteArchived)),
				SMessageDialog::FButton(LOCTEXT("CancelButton", "Cancel"))
					.SetPrimary(true)
					.SetFocus()
			});

		UE::MultiUserServer::FConcertServerUIModule::Get()
			.GetModalWindowManager()
			->ShowFakeModalWindow(Dialog);
	}
}

#undef LOCTEXT_NAMESPACE
