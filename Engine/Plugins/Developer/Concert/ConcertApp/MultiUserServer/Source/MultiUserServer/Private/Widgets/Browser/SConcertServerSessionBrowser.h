// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Session/Browser/SConcertSessionBrowser.h"
#include "Widgets/SConcertTabViewWithManagerBase.h"

class SConcertSessionBrowser;

namespace UE::MultiUserServer
{
	class FConcertServerSessionBrowserController;

	/** Shows a list of server sessions */
	class SConcertServerSessionBrowser : public SConcertTabViewWithManagerBase
	{
	public:
		
		static const FName SessionBrowserTabId;
	
		SLATE_BEGIN_ARGS(SConcertServerSessionBrowser) { }
			SLATE_ARGUMENT(TSharedPtr<SDockTab>, ConstructUnderMajorTab)
			SLATE_ARGUMENT(TSharedPtr<SWindow>, ConstructUnderWindow)
			SLATE_EVENT(FSessionDelegate, DoubleClickLiveSession)
			SLATE_EVENT(FSessionDelegate, DoubleClickArchivedSession)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FConcertServerSessionBrowserController> InController);

		void RequestRefreshListNextTick() { bRequestedRefresh = true; }
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
		{
			if (bRequestedRefresh)
			{
				SessionBrowser->RefreshSessionList();
				bRequestedRefresh = false;
			}

			SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
		}
	
	private:

		/** We can ask the controller about information and notify it about UI events. */
		TWeakPtr<FConcertServerSessionBrowserController> Controller;

		bool bRequestedRefresh = false;

		TSharedPtr<FText> SearchText;
		TSharedPtr<SConcertSessionBrowser> SessionBrowser;
		
		void CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FArguments& InArgs);
		
		TSharedRef<SDockTab> SpawnSessionBrowserTab(const FSpawnTabArgs& InTabArgs, FSessionDelegate DoubleClickLiveSession, FSessionDelegate DoubleClickArchivedSession);
		TSharedRef<SWidget> MakeSessionTableView(FSessionDelegate DoubleClickLiveSession, FSessionDelegate DoubleClickArchivedSession);
		
		void RequestDeleteSession(const TArray<TSharedPtr<FConcertSessionTreeItem>>& SessionItems);
		void DeleteSessionsWithFakeModalQuestion(const FText& Message, const TArray<TSharedPtr<FConcertSessionTreeItem>>& SessionItems);
	};
}
