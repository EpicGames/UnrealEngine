// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClientNetworkGraphs.h"

#include "ConcertServerStyle.h"
#include "SNetworkGraph.h"
#include "Widgets/Clients/Browser/Models/ITransferStatisticsModel.h"

#include "Math/UnitConversion.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SClientNetworkGraphs"

namespace UE::MultiUserServer 
{
	void SClientNetworkGraphs::Construct(const FArguments& InArgs, TSharedRef<ITransferStatisticsModel> InTransferStatsModel)
	{
		TransferStatisticsModel = MoveTemp(InTransferStatsModel);

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FConcertServerStyle::Get().GetBrush("Concert.Clients.ThumbnailCurveBackground"))
			[
				BuildGraphWidgets(InArgs)
			]
		];
	}

	TSharedRef<SWidget> SClientNetworkGraphs::BuildGraphWidgets(const FArguments& InArgs)
	{
		static_assert(static_cast<int32>(EConcertTransferStatistic::Count) == 2, "Update the styles");
		auto GetLineColor = [](EConcertTransferStatistic Stats)
		{
			switch (Stats)
			{
			case EConcertTransferStatistic::SentToClient: return FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.Sent.LineColor");
			case EConcertTransferStatistic::ReceivedFromClient: return FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.Received.LineColor");
			case EConcertTransferStatistic::Count:
			default:
				checkNoEntry();
				return FLinearColor{};
			}
		};
		auto GetFillColor = [](EConcertTransferStatistic Stats)
		{
			switch (Stats)
			{
			case EConcertTransferStatistic::SentToClient: return FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.Sent.FillColor");
			case EConcertTransferStatistic::ReceivedFromClient: return FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.Received.FillColor");
			case EConcertTransferStatistic::Count:
			default:
				checkNoEntry();
				return FLinearColor{};
			}
		};
		
		TSharedRef<SVerticalBox> Root = SNew(SVerticalBox);

		bool bNeedsSeparator = false;
		for (int32 i = 0; i < static_cast<int32>(EConcertTransferStatistic::Count); ++i)
		{
			if (bNeedsSeparator)
			{
				Root->AddSlot()
					.AutoHeight()
					.Padding(0.f, 1.f, 0.f, 0.f)
					[
						SNew(SSeparator)
						.Thickness(FConcertServerStyle::Get().GetFloat("Concert.Clients.NetworkGraph.GraphSeparatorLine.Thickness"))
						.ColorAndOpacity(FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.GraphSeparatorLine.LineColor"))
					];
			}
			bNeedsSeparator = true;

			const EConcertTransferStatistic Stat = static_cast<EConcertTransferStatistic>(i);
			TSharedPtr<SNetworkGraph> Graph;
			Root->AddSlot()
			[
				SAssignNew(Graph, SNetworkGraph)
				.RequestViewUpdate_Lambda([this, Stat]()
				{
					OnTransferStatisticsUpdated(Stat);
				})
				.CurrentNetworkValue(this, &SClientNetworkGraphs::GetStatTextValue, Stat)
				.DisplayedTimeRange(InArgs._TimeRange)
				.CurveColor(GetLineColor(Stat))
				.FillColor(GetFillColor(Stat))
			];
			Graphs.Add(Stat, Graph);
			TransferStatisticsModel->OnTransferTimelineUpdated(Stat).AddSP(this, &SClientNetworkGraphs::OnTransferStatisticsUpdated, Stat);
		}

		return Root;
	}

	FText SClientNetworkGraphs::GetStatTextValue(EConcertTransferStatistic Stat) const
	{
		const FString Label = [Stat]()
		{
			switch (Stat)
			{
			case EConcertTransferStatistic::SentToClient: return TEXT("S");
			case EConcertTransferStatistic::ReceivedFromClient: return TEXT("R");
			case EConcertTransferStatistic::Count: 
			default:
				checkNoEntry();
				return TEXT("Null");
			}
		}();

		const TArray<FConcertTransferSamplePoint>& TransferStatistics = TransferStatisticsModel->GetTransferStatTimeline(Stat);
		if (TransferStatistics.Num() > 0)
		{
			const uint64 BytesSent = TransferStatistics[TransferStatistics.Num() - 1].BytesTransferred;
			const FNumericUnit<uint64> TargetUnit = FUnitConversion::QuantizeUnitsToBestFit(BytesSent, EUnit::Bytes);
			return FText::Format(LOCTEXT("DataTextFmt", "{0}: {1} {2}"), FText::FromString(Label), TargetUnit.Value, FText::FromString(FUnitConversion::GetUnitDisplayString(TargetUnit.Units)));
		}
		
		return FText::Format(LOCTEXT("NoDataTextFmt", "{0}: n/a"), FText::FromString(Label));
	}

	void SClientNetworkGraphs::OnTransferStatisticsUpdated(EConcertTransferStatistic Stat)
	{
		const TArray<FConcertTransferSamplePoint>& Timeline = TransferStatisticsModel->GetTransferStatTimeline(Stat);
		Graphs[Stat]->UpdateView(Timeline.Num(), [&Timeline](uint32 Index)
		{
			return SNetworkGraph::FGraphData{ Timeline[Index].BytesTransferred, Timeline[Index].LocalTime };
		});
	}
}
#undef LOCTEXT_NAMESPACE
