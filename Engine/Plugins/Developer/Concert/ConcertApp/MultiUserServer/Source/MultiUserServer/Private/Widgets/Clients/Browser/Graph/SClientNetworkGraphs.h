// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace UE::MultiUserServer { enum class EConcertTransferStatistic; }

namespace UE::MultiUserServer
{
	class ITransferStatisticsModel;
	class SNetworkGraph;

	/** Displays send and receive graphs */
	class SClientNetworkGraphs : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SClientNetworkGraphs)
			: _TimeRange(FTimespan::FromSeconds(15))
		{}
			/** The last x time of traffic to display*/
			SLATE_ARGUMENT(FTimespan, TimeRange)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<ITransferStatisticsModel> InTransferStatsModel);

	private:

		/** Tells us the data to display */
		TSharedPtr<ITransferStatisticsModel> TransferStatisticsModel;

		TMap<EConcertTransferStatistic, TSharedPtr<SNetworkGraph>> Graphs;
		
		TSharedRef<SWidget> BuildGraphWidgets(const FArguments& InArgs);
		FText GetStatTextValue(EConcertTransferStatistic Stat) const;
		
		void OnTransferStatisticsUpdated(EConcertTransferStatistic Stat);
	};
}
