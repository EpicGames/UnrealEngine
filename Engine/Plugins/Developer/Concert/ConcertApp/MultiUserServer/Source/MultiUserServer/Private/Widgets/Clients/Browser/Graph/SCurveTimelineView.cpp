// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveTimelineView.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"
#include "SSimpleTimeSlider.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SCurveTimelineView"


void SCurveTimelineView::Construct( const SCurveTimelineView::FArguments& InArgs )
{
    ViewRange = InArgs._ViewRange;
	CurveColor = InArgs._CurveColor;
	FillColor = InArgs._FillColor;
	RenderFill = InArgs._RenderFill;
	DesiredSize = InArgs._DesiredSize;
	CurveData = InArgs._CurveData;

	// set clipping on by default, since the OnPaint function is drawing outside the bounds
	Clipping = EWidgetClipping::ClipToBounds;
}

int32 SCurveTimelineView::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayer = PaintCurve( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	return FMath::Max( NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) ) );
}


FVector2D SCurveTimelineView::ComputeDesiredSize( float ) const
{
	return DesiredSize.Get();
}

int32 SCurveTimelineView::PaintCurve(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	static const float LargeFrameTime = 1.125;  // ideally, we could check the recorded frame data for actual frame length
	 
	// convert time range to from rewind debugger times to profiler times
	TRange<double> DebugTimeRange = ViewRange.Get();
	FLinearColor LineColor = CurveColor.Get();
	
	SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen( DebugTimeRange, AllottedGeometry.GetLocalSize() );
	FVector2D Size = AllottedGeometry.GetLocalSize();

	if (TSharedPtr<FTimelineCurveData> Curve = CurveData.Get())
	{
		// Min/Max for normalization
		// maybe expose these as attributes too, so they are recomputed less frequently
		float MinValue = 1.0e10;
		float MaxValue = -1.0e10;

		if (bUseFixedRange)
		{
			MinValue = FixedRangeMin;
			MaxValue = FixedRangeMax;
		}
		else
		{
			for(auto& Point : Curve->Points)
			{
				MinValue = FMath::Min(MinValue, Point.Value);
				MaxValue = FMath::Max(MaxValue, Point.Value);
			}
		}
		
		float Range = FMath::Max(1.0,MaxValue - MinValue);
		
		auto &CurvePoints = Curve->Points;
		const int NumPoints = CurvePoints.Num();
		
		const ESlateDrawEffect LineDrawEffects = ESlateDrawEffect::NoBlending | ESlateDrawEffect::NoPixelSnapping;	

		if (NumPoints > 0)
		{
			if (RenderFill.Get())
			{
				// don't call this every frame
				FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*FAppStyle::GetBrush("Sequencer.SectionArea.Background"));
			
				const FSlateRenderTransform& Transform = AllottedGeometry.GetAccumulatedRenderTransform();
				FColor Color = FillColor.Get().ToFColor(true);

				// filled polygons
				TArray<FSlateVertex> Vertices;
				TArray<SlateIndex> Indices;
				Vertices.Reserve((NumPoints-1)*4);
				Indices.Reserve((NumPoints-1)*6);
				
				for(int i=1; i<NumPoints; i++)
				{
					const FTimelineCurveData::FCurvePoint& Point1 = CurvePoints[i-1];
					const FTimelineCurveData::FCurvePoint& Point2 = CurvePoints[i];
				
					if (Point2.Time - Point1.Time < LargeFrameTime)
					{
						float X1 = RangeToScreen.InputToLocalX(Point1.Time);
						const float NormalizedValue1 = (Point1.Value - MinValue) / Range;
						float Y1 = Size.Y - (Size.Y - 2) * NormalizedValue1;
						float X2 = RangeToScreen.InputToLocalX(Point2.Time);
						const float NormalizedValue2 = (Point2.Value - MinValue) / Range;
						float Y2 = Size.Y - (Size.Y - 2) * NormalizedValue2;
					
						Vertices.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(X1,Size.Y-1) ,FVector2f(0,0), Color));
						Vertices.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(X1,Y1),FVector2f(0,0), Color));
						Vertices.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(X2,Y2),FVector2f(0,0), Color));
						Vertices.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(X2,Size.Y-1),FVector2f(0,0), Color));
						
						int32 Index0 = Vertices.Num() - 4;
                        int32 Index1 = Vertices.Num() - 3;
                        int32 Index2 = Vertices.Num() - 2;
                        int32 Index3 = Vertices.Num() - 1;
                       
						Indices.Add(Index0);
						Indices.Add(Index1);
						Indices.Add(Index2);
						Indices.Add(Index0);
						Indices.Add(Index2);
						Indices.Add(Index3);
					}
					
					FSlateDrawElement::MakeCustomVerts(
								OutDrawElements,
										LayerId++,
										ResourceHandle,
										Vertices,
										Indices,
										nullptr,
										0,
										0);
				}
			}


			// Lines rendering
			
			TArray<FVector2D> Points;
			Points.Reserve(NumPoints);

			float PrevTime = CurvePoints[0].Time;
			for(int i=0; i<NumPoints; i++)
			{
				const FTimelineCurveData::FCurvePoint& Point = CurvePoints[i];
				if (Point.Time - PrevTime > LargeFrameTime && Points.Num()>1)
				{
					// break the line list - data has stopped and started again
					FSlateDrawElement::MakeLines(
								OutDrawElements,
										LayerId++,
										AllottedGeometry.ToPaintGeometry(),
										Points,
										LineDrawEffects,
										LineColor,
										false
										);
					Points.SetNum(0,EAllowShrinking::No);
				}
			
				float X = RangeToScreen.InputToLocalX(Point.Time);
				PrevTime = Point.Time;
				const float NormalizedValue = (Point.Value - MinValue) / Range;
				float Y = Size.Y - (Size.Y - 2) * NormalizedValue;
				Points.Add(FVector2D(X,Y));
			}
			
			FSlateDrawElement::MakeLines(
						OutDrawElements,
								LayerId,
								AllottedGeometry.ToPaintGeometry(),
								Points,
								LineDrawEffects,
								LineColor,
								true
								);


								
		}
	


	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
