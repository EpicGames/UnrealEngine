// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class FTimeSliderController;

/**
 * IMPORTANT: This code is duplicated and slighlty modified from Plugins/Animation/GameplayInsights.
 * The original widget code is supposed to make its way into ToolWidgets at which point this duplicate code should be removed.
 */


class SCurveTimelineView : public SCompoundWidget
{
public:
	struct FTimelineCurveData
	{
		struct FCurvePoint
		{
			double Time;
			float Value;
		};

		TArray<FCurvePoint> Points;
	};
	
	SLATE_BEGIN_ARGS(SCurveTimelineView)
		: _ViewRange(TRange<double>(0,10))
		, _DesiredSize(FVector2D(100.f,20.f))
		, _RenderFill(false)
	{}
    	/** View time range */
    	SLATE_ATTRIBUTE(TRange<double>, ViewRange);

		/** Data for curve to render */
		SLATE_ATTRIBUTE(TSharedPtr<FTimelineCurveData>, CurveData);

		/** Desired widget size */
		SLATE_ATTRIBUTE(FVector2D, DesiredSize);
	
		/** Curve Color */
		SLATE_ATTRIBUTE(FLinearColor, CurveColor);
	
		/** Fill Color */
		SLATE_ATTRIBUTE(FLinearColor, FillColor);
	
		/** Enable Fill rendering */
		SLATE_ATTRIBUTE(bool, RenderFill);
     
	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs );

	// render the curve with a fixed min/max range, rather than using the min/max from data
	void SetFixedRange(float Min, float Max)
	{
		bUseFixedRange = true;
		FixedRangeMin = Min;
		FixedRangeMax = Max;
	}

protected:
	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	
	int32 PaintCurve(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;
	
	TAttribute<TRange<double>> ViewRange;
	TAttribute<TSharedPtr<FTimelineCurveData>> CurveData;
	TAttribute<FVector2D> DesiredSize;
	TAttribute<FLinearColor> CurveColor;
	TAttribute<FLinearColor> FillColor;
	TAttribute<bool> RenderFill;
	
	const FSlateBrush* WhiteBrush;

	bool bUseFixedRange;
	float FixedRangeMin;
	float FixedRangeMax;
};

