// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetworkGraph.h"

#include "ConcertServerStyle.h"
#include "Math/UnitConversion.h"

#include "Widgets/Clients/Browser/Graph/SCurveTimelineView.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SNetworkGraph"

namespace UE::MultiUserServer
{
	void SNetworkGraph::Construct(const FArguments& InArgs)
	{
		check(InArgs._DisplayedTimeRange.IsBound() || InArgs._DisplayedTimeRange.IsSet());
		check(InArgs._RequestViewUpdate.IsBound());
		check(InArgs._TimeSliceSize > 0.f);
		
		DisplayedTimeRange = InArgs._DisplayedTimeRange;
		RequestViewUpdate = InArgs._RequestViewUpdate;
		MinimumYAxisRange = InArgs._MinimumYAxisRange;
		TimeSliceSize = InArgs._TimeSliceSize;
		TimeBucketAnchor = FDateTime::Now();
		
		CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
		
		ChildSlot
		[
			SNew(SOverlay)
	
			+SOverlay::Slot()
			.Padding(0.f, 5.f, 0.f, 0.f)
			[
				SAssignNew(Graph, SCurveTimelineView)
				.ViewRange(TRange<double>{ 0.0, DisplayedTimeRange.Get().GetTotalSeconds() })
				.CurveData(CurveData)
				.CurveColor(InArgs._CurveColor)
				.FillColor(InArgs._FillColor)
				.RenderFill(true)
			]
	
			+SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(InArgs._CurrentNetworkValue)
					.ColorAndOpacity(FColor::White)
				]
			]
			
			+SOverlay::Slot()
			[
				CreateHorizontalLines()
			]
		];

		SetTimerForForceRefresh();
	}

	using FCurvePoint = SCurveTimelineView::FTimelineCurveData::FCurvePoint;
	/** Distributes the points into time slices so the graph consists of n segements across the x-axis*/
	static TArray<FCurvePoint> DistributeIntoTimeBuckets(TArrayView<FCurvePoint> Curve, float TimeRange, float TimeSliceSize, const FDateTime& Now, const FDateTime& TimeAnchor);
	/** Makes a curve with corners flat so there are no longer an spiky corners on it. Imagine it like a curve of rectangles. */
	static TArray<FCurvePoint> FlattenCurve(TArrayView<FCurvePoint> Curve);

	void SNetworkGraph::UpdateView(uint32 Num, FGetGraphData GetDataFunc)
	{
		const FTimespan TimeRange = DisplayedTimeRange.Get();
		const double DisplayTimeRangInSeconds = TimeRange.GetTotalSeconds();
		const FDateTime LocalNow = FDateTime::Now();
		LastViewUpdate = LocalNow;

		TArray<FCurvePoint> Points;
		MaxDataInDisplayTime = MinimumYAxisRange;
		for (uint32 i = 0; i < Num; ++i)
		{
			const FGraphData GraphData = GetDataFunc(i);
			const FTimespan TimeSinceSample = LocalNow - GraphData.LocalTime;
			if (TimeSinceSample <= TimeRange)
			{
				MaxDataInDisplayTime = FMath::Max(MaxDataInDisplayTime, GraphData.Data);
				const double TimeLocation = DisplayTimeRangInSeconds - TimeSinceSample.GetTotalSeconds();
				const float Value = GraphData.Data;
			
				const SCurveTimelineView::FTimelineCurveData::FCurvePoint Point{ TimeLocation, Value };
				Points.Add(Point);
			}
		}
		
		Points = DistributeIntoTimeBuckets(Points, DisplayTimeRangInSeconds, TimeSliceSize, LocalNow, TimeBucketAnchor);
		CurveData->Points = FlattenCurve(Points);
		Graph->SetFixedRange(0, GetMaxRange());
	}

	static TArray<FCurvePoint> DistributeIntoTimeBuckets(TArrayView<FCurvePoint> Curve, float TimeRange, float TimeSliceSize, const FDateTime& Now, const FDateTime& TimeAnchor)
	{
		// The x-axis is divided into n equal segments of size TimeSliceSize

		// Step 1: Preallocate result
		const int32 ResultNum = TimeRange / TimeSliceSize;
		TArray<FCurvePoint> Result;
		Result.SetNumUninitialized(ResultNum);
		for (int32 ResultIdx = 0; ResultIdx < ResultNum; ++ResultIdx)
		{
			Result[ResultIdx] = { static_cast<float>(ResultIdx) * TimeSliceSize, 0 };
		}
		
		// Step 2: The times must be offset from a global time anchor point.
		// If we simply used the measurement time, we'd implicity use Now as time anchor;
		// That would mean that certain points would change buckets to the left or right as the graph is plotted over time (comment this code to see the effect).
		const FTimespan SinceAnchor = Now - TimeAnchor;
		const double BucketsSinceAnchor = SinceAnchor.GetTotalSeconds() / TimeSliceSize;
		const int32 NextBucketIndexAfterNow = static_cast<int32>(BucketsSinceAnchor) + 1;
		const FTimespan TimeUntilNextBucketAfterNow = FTimespan::FromSeconds(NextBucketIndexAfterNow * TimeSliceSize);
		const FDateTime StartTimeOfNextBucketAfterNow = TimeAnchor + TimeUntilNextBucketAfterNow;
		// Negative not positive: the offset time should end up in previous buckets... not in the next bucket which lies in the future
		const double TimeToOffset = -(StartTimeOfNextBucketAfterNow - Now).GetTotalSeconds();

		// Step 3: Sort the times into time buckets offsetting the times relative to TimeAnchor
		for (const FCurvePoint& CurvePoint : Curve)
		{
			// Due to micro gaps, e.g. 7.99999 and 9.00001 and sampling every second would assign
			// a value of 0 to the index corresponding to 8-9 seconds.
			// This is fixed by rounding the time.
			const int32 Index = FMath::Clamp<int32>(FMath::RoundToFloat(CurvePoint.Time + TimeToOffset) / TimeSliceSize, 0, ResultNum - 1);
			Result[Index].Value += CurvePoint.Value;
		}
		
		return Result;
	}
	
	static TArray<FCurvePoint> FlattenCurve(TArrayView<FCurvePoint> Curve)
	{
		TArray<FCurvePoint> Result;
		for (int32 i = 0; i < Curve.Num() - 1; ++i)
		{
			const FCurvePoint& Left = Curve[i];
			const FCurvePoint& Right = Curve[i + 1];

			Result.Add({ Left.Time, Left.Value });
			Result.Add({ Left.Time, Right.Value });
			Result.Add({ Right.Time, Right.Value });
		}
		return Result;
	}

	TSharedRef<SWidget> SNetworkGraph::CreateHorizontalLines() const
	{
		// There should be a line at 1/3 and 2/3 of the graph height (to guide the user's eye)
		// The line slots must take 1/3 of the space each while the top and the bottom buffer slots must take 1/6 each
		constexpr float LineFillHeight = 2; // = 0.3333f * 6
		constexpr float BufferFillHeight = 1; // = 0.1666f * 6 
		return SNew(SVerticalBox)
		
			+SVerticalBox::Slot()
			.FillHeight(BufferFillHeight)
			[
				SNullWidget::NullWidget
			]
		
			+SVerticalBox::Slot()
			.FillHeight(LineFillHeight)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this](){ return GetBytesDisplayText(2 * GetMaxRange() / 3); })
					.ColorAndOpacity(FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.HorizontalHelperLine.TextColor"))
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(2.f, 0.f, 0.f, 0.f)
				.VAlign(VAlign_Center)
				[
					SNew(SSeparator)
					.Thickness(FConcertServerStyle::Get().GetFloat("Concert.Clients.NetworkGraph.HorizontalHelperLine.Thickness"))
					.ColorAndOpacity(FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.HorizontalHelperLine.LineColor"))
				]
			]

			+SVerticalBox::Slot()
			.FillHeight(LineFillHeight)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this](){ return GetBytesDisplayText(GetMaxRange() / 3); })
					.ColorAndOpacity(FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.HorizontalHelperLine.TextColor"))
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(2.f, 0.f, 0.f, 0.f)
				.VAlign(VAlign_Center)
				[
					SNew(SSeparator)
					.Thickness(FConcertServerStyle::Get().GetFloat("Concert.Clients.NetworkGraph.HorizontalHelperLine.Thickness"))
					.ColorAndOpacity(FConcertServerStyle::Get().GetColor("Concert.Clients.NetworkGraph.HorizontalHelperLine.LineColor"))
				]
			]

			+SVerticalBox::Slot()
			.FillHeight(BufferFillHeight)
			[
				SNullWidget::NullWidget
			];
	}

	FText SNetworkGraph::GetBytesDisplayText(uint64 Data) const
	{
		const FNumericUnit<uint64> TargetUnit = FUnitConversion::QuantizeUnitsToBestFit(Data, EUnit::Bytes);
		return FText::Format(LOCTEXT("BytesFmt", "{0} {1}"), TargetUnit.Value, FText::FromString(FUnitConversion::GetUnitDisplayString(TargetUnit.Units)));
	}

	void SNetworkGraph::SetTimerForForceRefresh()
	{
		RegisterActiveTimer(TimeSliceSize, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
		{
			const FTimespan TimeSinceLastUpdate = LastViewUpdate - FDateTime::Now();
			if (TimeSinceLastUpdate.GetTotalSeconds() < TimeSliceSize)
			{
				RequestViewUpdate.Execute();
			}
			return EActiveTimerReturnType::Continue;
		}));
	}
}

#undef LOCTEXT_NAMESPACE
