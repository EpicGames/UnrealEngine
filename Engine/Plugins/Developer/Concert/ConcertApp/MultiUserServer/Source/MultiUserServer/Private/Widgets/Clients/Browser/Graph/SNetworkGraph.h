// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCurveTimelineView.h"

namespace UE::MultiUserServer
{
	/**
	 * Displays a graph from a source of floats
	 */
	class SNetworkGraph : public SCompoundWidget
	{
	public:

		DECLARE_DELEGATE(FRequestViewUpdate);
		
		struct FGraphData
		{
			uint64 Data;
			FDateTime LocalTime;
		};
		
		using FGetGraphData = TFunctionRef<FGraphData(uint32 Index)>;

		SLATE_BEGIN_ARGS(SNetworkGraph)
			: _MinimumYAxisRange(FMath::Pow(2.f, 13.f)) // 8 kilobytes
			, _TimeSliceSize(1.f)
		{}
			/** Displayed at the top-right of the graph, e.g. S: 128 kB */
			SLATE_ATTRIBUTE(FText, CurrentNetworkValue)
			/** All data within the last DisplayedTimeRange.GetTotalSeconds() will be displayed. */
			SLATE_ATTRIBUTE(FTimespan, DisplayedTimeRange)

			SLATE_ARGUMENT(FLinearColor, CurveColor)
			SLATE_ARGUMENT(FLinearColor, FillColor)
			SLATE_ARGUMENT(uint64, MinimumYAxisRange)
			/** The curve is divided into even segments along the x-axis. This specifies the number of seconds each segment is wide. */
			SLATE_ARGUMENT(float, TimeSliceSize)

			/** Called when it has been too long since UpdateView has last been called. Causes UpdateView to be called. */
			SLATE_EVENT(FRequestViewUpdate, RequestViewUpdate)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/** Updates the graph view to display the given data. */
		void UpdateView(uint32 Num, FGetGraphData GetDataFunc);

	private:

		/** All data within the last DisplayedTimeRange.GetTotalSeconds() will be displayed. */
		TAttribute<FTimespan> DisplayedTimeRange;
		/** Called when it has been too long since UpdateView has last been called. Causes UpdateView to be called. */
		FRequestViewUpdate RequestViewUpdate;

		/** Renders the graph */
		TSharedPtr<SCurveTimelineView> Graph;
		/** The data displayed in Graph*/
		TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;
		
		/** Heightest point in CurveData. Used to correctly set y-axis scale */
		uint64 MaxDataInDisplayTime;
		/** Used to regularily request updates if the network does not provide them (e.g. because there just was no traffic). */
		FDateTime LastViewUpdate;
		
		/**
		 * The graph is constantly scaled to fit the biggest value in the point set.
		 * This variable artifically sets the minimum value to which we scale.
		 */
		uint64 MinimumYAxisRange;
		/**
		 * The curve is divided into even segments along the x-axis.
		 * This specifies the number of seconds each segment is wide.
		 */
		float TimeSliceSize;
		/** All time slice buckets are relative to this time which is taken when the widget is constructed. */
		FDateTime TimeBucketAnchor;
		

		TSharedRef<SWidget> CreateHorizontalLines() const;
		FText GetBytesDisplayText(uint64 Data) const;

		/** Updates the graph in an interval of TimeSliceSize in case no network data is received in that time. */
		void SetTimerForForceRefresh();

		uint64 GetMaxRange() const { return MaxDataInDisplayTime * 1.15; }
	};
}


