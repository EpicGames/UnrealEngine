// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientBrowserItem.h"

#include "Widgets/Clients/Browser/Models/IClientNetworkStatisticsModel.h"
#include "Widgets/Clients/Browser/Models/Transfer/ClientTransferStatisticsModel.h"
#include "Widgets/Clients/Browser/Models/Transfer/TransferStatisticsModel.h"
#include "Widgets/Clients/Util/EndpointToUserNameCache.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FClientBrowserItem"

namespace UE::MultiUserServer
{
	FClientBrowserItem::FClientBrowserItem(TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel, TSharedRef<FEndpointToUserNameCache> UserLookup, const FMessageAddress& ClientAddress, const FGuid& MessageNodeId)
		: NetworkStatisticsModel(MoveTemp(NetworkStatisticsModel))
		, UserLookup(MoveTemp(UserLookup))
		, TransferStatisticsModel(MakeShared<FClientTransferStatisticsModel>(ClientAddress))
		, ClientAddress(ClientAddress)
		, MessageNodeId(MessageNodeId)
	{}
	
	FString FClientBrowserItem::GetDisplayName() const
	{
		if (const TOptional<FConcertClientInfo> ClientInfo = UserLookup->GetClientInfoFromNodeId(MessageNodeId))
		{
			return ClientInfo->DisplayName;
		}
		
		const FString NodeIdAsString = MessageNodeId.ToString(EGuidFormats::DigitsWithHyphens);
		int32 Index = INDEX_NONE;
		const bool bFound = NodeIdAsString.FindChar('-', Index);
		// Avoid making the name too long by only showing the first few digits of the node ID
		return FString::Printf(TEXT("Admin (%s)"), bFound ? *NodeIdAsString.Left(Index) : *NodeIdAsString); 
	}

	FText FClientBrowserItem::GetToolTip() const
	{
		if (const TOptional<FConcertClientInfo> ClientInfo = UserLookup->GetClientInfoFromNodeId(MessageNodeId))
		{
			return FText::Format(
				LOCTEXT("Name.Available.TooltipFmt", "NodeID: {0}\nAddress ID: {1}"),
				FText::FromString(MessageNodeId.ToString(EGuidFormats::DigitsWithHyphens)),
				FText::FromString(ClientAddress.ToString())
				);
		}
		return FText::Format(
			LOCTEXT("Name.NotAvailable.TooltipFmt", "This client's display information becomes available after joining a session.\nNodeID: {0}\nAddress ID: {1}"),
			FText::FromString(MessageNodeId.ToString(EGuidFormats::DigitsWithHyphens)),
			FText::FromString(ClientAddress.ToString())
			);
	}

	TSharedRef<ITransferStatisticsModel> FClientBrowserItem::GetTransferStatistics() const
	{
		return TransferStatisticsModel;
	}

	TOptional<FMessageTransportStatistics> FClientBrowserItem::GetLatestNetworkStatistics() const
	{
		return NetworkStatisticsModel->GetLatestNetworkStatistics(ClientAddress);
	}

	bool FClientBrowserItem::IsOnline() const
	{
		return NetworkStatisticsModel->IsOnline(ClientAddress);
	}
}

#undef LOCTEXT_NAMESPACE
