// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertBrowserItem.h"
#include "IMessageContext.h"

namespace UE::MultiUserServer { class IClientNetworkStatisticsModel; }
struct FConcertClientInfo;

class FEndpointToUserNameCache;

namespace UE::MultiUserServer
{
	class ITransferStatisticsModel;
	class FClientTransferStatisticsModel;

	DECLARE_DELEGATE_RetVal(TOptional<FConcertClientInfo>, FGetClientInfo);
	
	class FClientBrowserItem : public FConcertBrowserItemCommonImpl
	{
	public:

		FClientBrowserItem(TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel, TSharedRef<FEndpointToUserNameCache> UserLookup, const FMessageAddress& ClientAddress, const FGuid& MessageNodeId);
		
		void OnJoinSession(const FGuid& SessionId) { CurrentSession = SessionId; }
		void OnLeaveSession() { CurrentSession.Reset(); }
		const TOptional<FGuid>& GetCurrentSession() const { return CurrentSession; }
		const FGuid& GetMessageNodeId() const { return MessageNodeId; }

		void OnClientDisconnected() { bIsDisconnected = true; }
		void OnClientReconnected() { bIsDisconnected = false; }
		/** Whether this client is no longer connected with the server */
		bool IsDisconnected() const { return bIsDisconnected; }
		
		virtual FString GetDisplayName() const override;
		virtual FText GetToolTip() const override;
		virtual TSharedRef<ITransferStatisticsModel> GetTransferStatistics() const override;
		virtual TOptional<FMessageTransportStatistics> GetLatestNetworkStatistics() const override;
		virtual bool IsOnline() const override;

	private:

		TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel;
		
		/** Used to gets the client's name once it becomes available */
		TSharedRef<FEndpointToUserNameCache> UserLookup;

		/** Keeps track of the clients transfer statistics */
		TSharedRef<FClientTransferStatisticsModel> TransferStatisticsModel;

		/** Address of this client as used by the UDP messaging sytem */
		FMessageAddress ClientAddress;
		
		/** ID of this client in the UDP messaging system */
		FGuid MessageNodeId;
		
		/** The session this client currently is in */
		TOptional<FGuid> CurrentSession;

		/** Whether this client is no longer connected with the server */
		bool bIsDisconnected = false;
	};
}
