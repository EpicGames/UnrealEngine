// Copyright Epic Games, Inc. All Rights Reserved.

#include "IConcertBrowserItem.h"

#include "Widgets/Clients/Browser/Models/IClientNetworkStatisticsModel.h"

#include "INetworkMessagingExtension.h"

namespace UE::MultiUserServer
{
	void FConcertBrowserItemCommonImpl::SetDisplayMode(EConcertBrowserItemDisplayMode Value)
	{
		ItemDisplayMode = Value;
	}
	
	EConcertBrowserItemDisplayMode FConcertBrowserItemCommonImpl::GetDisplayMode() const
	{
		return ItemDisplayMode;
	}

	void FConcertBrowserItemCommonImpl::AppendSearchTerms(TArray<FString>& SearchTerms) const
	{
		SearchTerms.Add(GetDisplayName());
		const TOptional<FMessageTransportStatistics> Stats = GetLatestNetworkStatistics();
		SearchTerms.Add(NetworkStatistics::FormatIPv4AsString(Stats));
		
		if (Stats)
		{
			SearchTerms.Add(NetworkStatistics::FormatTotalBytesSent(*Stats));
			SearchTerms.Add(NetworkStatistics::FormatTotalBytesReceived(*Stats));
			SearchTerms.Add(NetworkStatistics::FormatAverageRTT(*Stats));
			SearchTerms.Add(NetworkStatistics::FormatBytesInflight(*Stats));
			SearchTerms.Add(NetworkStatistics::FormatTotalBytesLost(*Stats));
		}
	}
}
