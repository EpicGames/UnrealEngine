// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

template <typename OptionalType> struct TOptional;

struct FMessageTransportStatistics;
struct FMessageAddress;

namespace UE::MultiUserServer
{
	class IClientNetworkStatisticsModel;
	class ITransferStatisticsModel;

	enum class EConcertBrowserItemDisplayMode : uint8
	{
		/** Displays the sent and read packets */
		NetworkGraph = 0,
		/** Displays a table showing MessageId, Sent, Acked, and Size updated in realtime for outbound UDP segments. */
		OutboundSegementTable = 1,
		/** Displays a table showing MessageId, Received, and Size updated in realtime for inbound UDP segments. */
		InboundSegmentTable = 2
	};

	/** Implemented by items that can be displayed in the Concert Network Browser */
	class IConcertBrowserItem
	{
	public:

		/** The title of the thumbnail */
		virtual FString GetDisplayName() const = 0;
		/** What to display when the thumbnail is hovered */
		virtual FText GetToolTip() const = 0;
		
		virtual void SetDisplayMode(EConcertBrowserItemDisplayMode Value) = 0;
		virtual EConcertBrowserItemDisplayMode GetDisplayMode() const = 0;

		/** The network statistics for this item */
		virtual TSharedRef<ITransferStatisticsModel> GetTransferStatistics() const = 0;

		/** Adds terms this item can be searched by */
		virtual void AppendSearchTerms(TArray<FString>& SearchTerms) const = 0;

		/** Gets the latest network statistics for this item if they are available; most likely unavailable when IsOnline returns false. */
		virtual TOptional<FMessageTransportStatistics> GetLatestNetworkStatistics() const = 0;
		/** Whether this client is currently reachable */
		virtual bool IsOnline() const = 0;

		virtual ~IConcertBrowserItem() = default;
	};

	/** Util for implementing shared logic. */
	class FConcertBrowserItemCommonImpl : public IConcertBrowserItem
	{
	public:
		
		virtual void SetDisplayMode(EConcertBrowserItemDisplayMode Value) override;
		virtual EConcertBrowserItemDisplayMode GetDisplayMode() const override;
		virtual void AppendSearchTerms(TArray<FString>& SearchTerms) const override;

	private:

		EConcertBrowserItemDisplayMode ItemDisplayMode = EConcertBrowserItemDisplayMode::NetworkGraph;
	};
}
