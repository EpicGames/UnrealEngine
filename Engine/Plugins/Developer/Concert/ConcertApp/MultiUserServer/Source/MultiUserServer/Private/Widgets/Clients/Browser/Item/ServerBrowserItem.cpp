// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerBrowserItem.h"

#include "IConcertServer.h"
#include "IConcertSession.h"
#include "Widgets/Clients/Browser/Models/IClientNetworkStatisticsModel.h"
#include "Widgets/Clients/Browser/Models/Transfer/ServerTransferStatisticsModel.h"
#include "Widgets/Clients/Browser/Models/Transfer/TransferStatisticsModel.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FServerBrowserItem"

namespace UE::MultiUserServer
{
	FServerBrowserItem::FServerBrowserItem(TSharedRef<IConcertServer> Server, TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel)
		: Server(MoveTemp(Server))
		, NetworkStatisticsModel(MoveTemp(NetworkStatisticsModel))
		, ServerTransferStats(MakeShared<FServerTransferStatisticsModel>())
	{}

	FString FServerBrowserItem::GetDisplayName() const
	{
		return TEXT("Server");
	}

	FText FServerBrowserItem::GetToolTip() const
	{
		return LOCTEXT("ServerTooltip", "The server sums up all connected client statistics.");
	}

	TSharedRef<ITransferStatisticsModel> FServerBrowserItem::GetTransferStatistics() const
	{
		return ServerTransferStats;
	}

	TOptional<FMessageTransportStatistics> FServerBrowserItem::GetLatestNetworkStatistics() const
	{
		TArray<FMessageTransportStatistics> AllStats;
		for (const TSharedPtr<IConcertServerSession>& Session : Server->GetLiveSessions())
		{
			for (const FConcertSessionClientInfo& ClientInfo : Session->GetSessionClients())
			{
				const FMessageAddress ClientAddress = Session->GetClientAddress(ClientInfo.ClientEndpointId);
				const TOptional<FMessageTransportStatistics> Stats = NetworkStatisticsModel->GetLatestNetworkStatistics(ClientAddress);
				if (Stats)
				{
					AllStats.Add(*Stats);
				}
			}
			
		}

		for (const FConcertEndpointContext& RemoteClientAdminEndpoints : Server->GetRemoteAdminEndpoints())
		{
			const TOptional<FMessageTransportStatistics> Stats = NetworkStatisticsModel->GetLatestNetworkStatistics(Server->GetRemoteAddress(RemoteClientAdminEndpoints.EndpointId));
			if (Stats)
			{
				AllStats.Add(*Stats);
			}
		}

		FMessageTransportStatistics Result = SumUpStats(AllStats);
		Result.IPv4AsString = TEXT("localhost"); // Should probably look up the IP address values ...
		return Result;
	}

	FMessageTransportStatistics FServerBrowserItem::SumUpStats(const TArray<FMessageTransportStatistics>& StatArray) const
	{
		FMessageTransportStatistics Result;

		for (const FMessageTransportStatistics& Stats : StatArray)
		{
			Result.TotalBytesSent += Stats.TotalBytesSent;
			Result.TotalBytesLost += Stats.TotalBytesLost;
			Result.TotalBytesReceived += Stats.TotalBytesReceived;
			Result.BytesInflight += Stats.BytesInflight;
			Result.PacketsSent += Stats.PacketsSent;
			Result.PacketsLost += Stats.PacketsLost;
			Result.PacketsAcked += Stats.PacketsAcked;
			Result.PacketsReceived += Stats.PacketsReceived;
			Result.PacketsInFlight += Stats.PacketsInFlight;
			Result.AverageRTT += Stats.AverageRTT;
		}

		if (StatArray.Num() > 0)
		{
			Result.AverageRTT /= StatArray.Num();
		}
		Result.WindowSize = 0;

		return Result;
	}
}

#undef LOCTEXT_NAMESPACE
