// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertBrowserItem.h"

namespace UE::MultiUserServer { class IClientNetworkStatisticsModel; }

class IConcertServer;

namespace UE::MultiUserServer
{
	class FServerTransferStatisticsModel;

	class FServerBrowserItem : public FConcertBrowserItemCommonImpl
	{
	public:

		FServerBrowserItem(TSharedRef<IConcertServer> Server, TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel);
	
		virtual FString GetDisplayName() const override;
		virtual FText GetToolTip() const override;
		virtual TSharedRef<ITransferStatisticsModel> GetTransferStatistics() const override;
		virtual TOptional<FMessageTransportStatistics> GetLatestNetworkStatistics() const override;
		virtual bool IsOnline() const override { return true;}

	private:

		TSharedRef<IConcertServer> Server;
		TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel;
		TSharedRef<FServerTransferStatisticsModel> ServerTransferStats;

		FMessageTransportStatistics SumUpStats(const TArray<FMessageTransportStatistics>& Stats) const;
	};
}

