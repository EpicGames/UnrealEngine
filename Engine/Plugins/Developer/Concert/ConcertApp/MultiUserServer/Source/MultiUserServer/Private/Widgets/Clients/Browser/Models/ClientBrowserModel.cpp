// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientBrowserModel.h"

#include "ConcertMessages.h"
#include "ConcertServerEvents.h"
#include "IConcertServer.h"
#include "IConcertSession.h"
#include "Widgets/Clients/Browser/Item/ClientBrowserItem.h"

#include "Features/IModularFeatures.h"
#include "INetworkMessagingExtension.h"
#include "Widgets/Clients/Browser/Item/IConcertBrowserItem.h"

UE::MultiUserServer::FClientBrowserModel::FClientBrowserModel(TSharedRef<IConcertServer> InServer, TSharedRef<FEndpointToUserNameCache> ClientInfoCache, TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel)
	: Server(MoveTemp(InServer))
	, ClientInfoCache(MoveTemp(ClientInfoCache))
	, NetworkStatisticsModel(MoveTemp(NetworkStatisticsModel))
{
	ConcertServerEvents::OnLiveSessionCreated().AddRaw(this, &FClientBrowserModel::OnLiveSessionCreated);
	ConcertServerEvents::OnLiveSessionDestroyed().AddRaw(this, &FClientBrowserModel::OnLiveSessionDestroyed);
	for (const TSharedPtr<IConcertServerSession>& LiveSession : Server->GetLiveSessions())
	{
		SubscribeToClientConnectionEvents(LiveSession.ToSharedRef());
	}

	for (const FConcertEndpointContext& Context : Server->GetRemoteAdminEndpoints())
	{
		AddClientAdminEndpoint(Context);
	}
	Server->OnRemoteEndpointConnectionChanged().AddRaw(this, &FClientBrowserModel::OnAdminClientEndpointConnectionChanged);
}

UE::MultiUserServer::FClientBrowserModel::~FClientBrowserModel()
{
	ConcertServerEvents::OnLiveSessionCreated().RemoveAll(this);
	ConcertServerEvents::OnLiveSessionDestroyed().RemoveAll(this);
	for (const TSharedPtr<IConcertServerSession>& LiveSession : Server->GetLiveSessions())
	{
		UnsubscribeFromClientConnectionEvents(LiveSession.ToSharedRef());
	}
	
	Server->OnRemoteEndpointConnectionChanged().RemoveAll(this);
}

TSet<FGuid> UE::MultiUserServer::FClientBrowserModel::GetSessions() const
{
	const TArray<TSharedPtr<IConcertServerSession>> LiveSessions = Server->GetLiveSessions();
	TSet<FGuid> Result;
	Algo::Transform(LiveSessions, Result, [](const TSharedPtr<IConcertServerSession>& ServerSession){ return ServerSession->GetId(); });
	return Result;
}

TOptional<FConcertSessionInfo> UE::MultiUserServer::FClientBrowserModel::GetSessionInfo(const FGuid& SessionID) const
{
	const TSharedPtr<IConcertServerSession> LiveSession = Server->GetLiveSession(SessionID);
	return LiveSession
		? LiveSession->GetSessionInfo()
		: TOptional<FConcertSessionInfo>();
}

void UE::MultiUserServer::FClientBrowserModel::SetKeepClientsAfterDisconnect(bool bNewValue)
{
	if (bNewValue == bKeepClientsAfterDisconnect)
	{
		return;
	}

	bKeepClientsAfterDisconnect = bNewValue;

	if (bKeepClientsAfterDisconnect)
	{
		return;
	}
	for (auto It = Clients.CreateIterator(); It; ++It)
	{
		if (It->Get()->IsDisconnected())
		{
			OnClientListChanged().Broadcast(*It, EClientUpdateType::Removed);
			It.RemoveCurrent();
		}
	}
}

void UE::MultiUserServer::FClientBrowserModel::OnLiveSessionCreated(bool bSuccess, const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession)
{
	if (bSuccess)
	{
		SubscribeToClientConnectionEvents(InLiveSession);
	}
	
	OnSessionCreatedEvent.Broadcast(InLiveSession->GetId());
}

void UE::MultiUserServer::FClientBrowserModel::OnLiveSessionDestroyed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession) const
{
	OnSessionCreatedEvent.Broadcast(InLiveSession->GetId());
}

void UE::MultiUserServer::FClientBrowserModel::SubscribeToClientConnectionEvents(const TSharedRef<IConcertServerSession>& InLiveSession)
{
	InLiveSession->OnSessionClientChanged().AddRaw(this, &FClientBrowserModel::OnClientListUpdated);
}

void UE::MultiUserServer::FClientBrowserModel::UnsubscribeFromClientConnectionEvents(const TSharedRef<IConcertServerSession>& InLiveSession) const
{
	InLiveSession->OnSessionClientChanged().RemoveAll(this);
}

void UE::MultiUserServer::FClientBrowserModel::OnClientListUpdated(IConcertServerSession& Session, EConcertClientStatus Status, const FConcertSessionClientInfo& ClientInfo)
{
	switch (Status)
	{
		case EConcertClientStatus::Connected:
			OnClientJoinSession(Session, ClientInfo.ClientEndpointId);
			break;
		case EConcertClientStatus::Disconnected:
			OnClientLeaveSession(Session, ClientInfo.ClientEndpointId);
			break;
		case EConcertClientStatus::Updated:
			break;
		default:
			checkNoEntry();
	}
}

void UE::MultiUserServer::FClientBrowserModel::OnClientJoinSession(IConcertServerSession& Session, const FGuid& EndpointId)
{
	UpdateClientSessionId(Session, EndpointId, Session.GetId());
}

void UE::MultiUserServer::FClientBrowserModel::OnClientLeaveSession(IConcertServerSession& Session, const FGuid& EndpointId)
{
	UpdateClientSessionId(Session, EndpointId, {});
}

void UE::MultiUserServer::FClientBrowserModel::UpdateClientSessionId(IConcertServerSession& Session, const FGuid& EndpointId, TOptional<FGuid> SessionId)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (!ensure(ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName)))
	{
		return;
	}
	const INetworkMessagingExtension& Extension = ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
	
	const FMessageAddress Address = Session.GetClientAddress(EndpointId);
	// Invalid if client disconnects while in a session
	if (!Address.IsValid())
	{
		return;
	}
	
	const FGuid NodeId = Extension.GetNodeIdFromAddress(Address);
	const int32 Index = Clients.IndexOfByPredicate([&NodeId](const TSharedPtr<FClientBrowserItem>& Item)
	{
		return Item->GetMessageNodeId() == NodeId;
	});
	if (Clients.IsValidIndex(Index))
	{
		if (SessionId)
		{
			Clients[Index]->OnJoinSession(*SessionId);
		}
		else
		{
			Clients[Index]->OnLeaveSession();
		}
	}
}

void UE::MultiUserServer::FClientBrowserModel::OnAdminClientEndpointConnectionChanged(const FConcertEndpointContext& Context, EConcertRemoteEndpointConnection ConnectionState)
{
	switch (ConnectionState)
	{
	case EConcertRemoteEndpointConnection::Discovered:
		AddClientAdminEndpoint(Context);
		break;
	case EConcertRemoteEndpointConnection::TimedOut:
	case EConcertRemoteEndpointConnection::ClosedRemotely:
		RemoveClientAdminEndpoint(Context);
		break;
	default:
		checkNoEntry();
	}
}

void UE::MultiUserServer::FClientBrowserModel::AddClientAdminEndpoint(const FConcertEndpointContext& Context)
{
	const FMessageAddress Address = Server->GetRemoteAddress(Context.EndpointId);

	if (!ensure(Address.IsValid()))
	{
		return;
	}
	
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const INetworkMessagingExtension* MessagingExtension = ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName)
		? &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName)
		: nullptr;
	if (!ensure(MessagingExtension))
	{
		return;
	}
	const FGuid NodeId = MessagingExtension->GetNodeIdFromAddress(Address);
	const TSharedPtr<FClientBrowserItem>* ExistingItem = Clients.FindByPredicate([&NodeId](const TSharedPtr<FClientBrowserItem>& Item)
	{
		return Item->GetMessageNodeId() == NodeId;
	});
	if (ExistingItem)
	{
		ExistingItem->Get()->OnClientReconnected();
		return;
	}
	
	const TSharedRef<FClientBrowserItem> Item = MakeShared<FClientBrowserItem>(
		NetworkStatisticsModel,
		ClientInfoCache,
		Address,
		NodeId
		);
	Clients.Add(Item);
	EndpointToNodeId.Add(Context.EndpointId, NodeId);
	OnClientListChangedEvent.Broadcast(Item, EClientUpdateType::Added);
}

void UE::MultiUserServer::FClientBrowserModel::RemoveClientAdminEndpoint(const FConcertEndpointContext& Context)
{
	const FMessageNodeId* NodeId = EndpointToNodeId.Find(Context.EndpointId);
	if (!NodeId)
	{
		return;
	}
	
	const int32 Index = Clients.IndexOfByPredicate([NodeId](const TSharedPtr<FClientBrowserItem>& Item)
	{
		return Item->GetMessageNodeId() == *NodeId;
	});
	
	if (ensure(Clients.IsValidIndex(Index)))
	{
		EndpointToNodeId.Remove(Context.EndpointId);
		const TSharedPtr<FClientBrowserItem> Item = Clients[Index]; 
		if (bKeepClientsAfterDisconnect)
		{
			Item->OnClientDisconnected();
			return;
		}
		
		Clients.RemoveAt(Index);
		OnClientListChangedEvent.Broadcast(Item, EClientUpdateType::Removed);
	}
}
