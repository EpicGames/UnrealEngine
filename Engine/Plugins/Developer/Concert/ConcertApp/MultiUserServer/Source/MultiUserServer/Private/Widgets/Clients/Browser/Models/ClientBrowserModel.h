// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientBrowserModel.h"
#include "Misc/Guid.h"

enum class EConcertClientStatus : uint8;
struct FConcertSessionClientInfo;

enum class EConcertRemoteEndpointConnection : uint8;
struct FConcertEndpointContext;
class FEndpointToUserNameCache;
class IConcertServerSession;
class IConcertServer;

namespace UE::MultiUserServer
{
	class IClientNetworkStatisticsModel;

	class FClientBrowserModel : public IClientBrowserModel
	{
	public:

		FClientBrowserModel(TSharedRef<IConcertServer> InServer, TSharedRef<FEndpointToUserNameCache> ClientInfoCache, TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel);
		virtual ~FClientBrowserModel() override;

		//~ Begin IClientBrowserModel Interface
		virtual TSet<FGuid> GetSessions() const override;
		virtual TOptional<FConcertSessionInfo> GetSessionInfo(const FGuid& SessionID) const override;
		virtual const TArray<TSharedPtr<FClientBrowserItem>>& GetItems() const override { return Clients; }
		virtual void SetKeepClientsAfterDisconnect(bool bNewValue) override;
		virtual bool ShouldKeepClientsAfterDisconnect() const override { return bKeepClientsAfterDisconnect; }
		virtual FOnClientListChanged& OnClientListChanged() override { return OnClientListChangedEvent; }
		virtual FOnSessionListChanged& OnSessionCreated() override { return OnSessionCreatedEvent; }
		virtual FOnSessionListChanged& OnSessionDestroyed() override { return OnSessionDestroyedEvent; }
		//~ End IClientBrowserModel Interface

	private:

		using FClientEndpointId = FGuid;
		using FMessageNodeId = FGuid;

		TSharedRef<IConcertServer> Server;
		TSharedRef<FEndpointToUserNameCache> ClientInfoCache;
		TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel;

		TArray<TSharedPtr<FClientBrowserItem>> Clients;
		/** Needed for when an admin endpoint disconnects - querying won't work anymore then. */
		TMap<FClientEndpointId, FMessageNodeId> EndpointToNodeId;

		bool bKeepClientsAfterDisconnect = true;

		FOnClientListChanged OnClientListChangedEvent;
		FOnSessionListChanged OnSessionCreatedEvent;
		FOnSessionListChanged OnSessionDestroyedEvent;

		// Session events
		void OnLiveSessionCreated(bool bSuccess, const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession);
		void OnLiveSessionDestroyed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession) const;
		void SubscribeToClientConnectionEvents(const TSharedRef<IConcertServerSession>& InLiveSession);
		void UnsubscribeFromClientConnectionEvents(const TSharedRef<IConcertServerSession>& InLiveSession) const;

		// Handle session clients
		void OnClientListUpdated(IConcertServerSession& Session, EConcertClientStatus Status, const FConcertSessionClientInfo& ClientInfo);
		void OnClientJoinSession(IConcertServerSession& Session, const FGuid& EndpointId);
		void OnClientLeaveSession(IConcertServerSession& Session, const FGuid& EndpointId);
		void UpdateClientSessionId(IConcertServerSession& Session, const FGuid& EndpointId, TOptional<FGuid> SessionId);

		void OnAdminClientEndpointConnectionChanged(const FConcertEndpointContext& Context, EConcertRemoteEndpointConnection ConnectionState);
		void AddClientAdminEndpoint(const FConcertEndpointContext& Context);
		void RemoveClientAdminEndpoint(const FConcertEndpointContext& Context);
	};
}


