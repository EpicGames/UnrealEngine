// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientNetworkStatisticsModel.h"

#include "Features/IModularFeatures.h"
#include "INetworkMessagingExtension.h"
#include "Misc/ScopeExit.h"

namespace UE::MultiUserServer::Private
{
	static INetworkMessagingExtension* GetMessagingStatistics()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (IsInGameThread())
		{
			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}
		else
		{
			ModularFeatures.LockModularFeatureList();
			ON_SCOPE_EXIT
			{
				ModularFeatures.UnlockModularFeatureList();
			};
			
			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}
		

		ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
		return nullptr;
	}
}

TOptional<FMessageTransportStatistics> UE::MultiUserServer::FClientNetworkStatisticsModel::GetLatestNetworkStatistics(const FMessageAddress& ClientAddress) const
{
	if (INetworkMessagingExtension* Statistics = Private::GetMessagingStatistics())
	{
		const FGuid NodeId = Statistics->GetNodeIdFromAddress(ClientAddress);
		return NodeId.IsValid() ? Statistics->GetLatestNetworkStatistics(NodeId) : TOptional<FMessageTransportStatistics>();
	}
	return {};
}

bool UE::MultiUserServer::FClientNetworkStatisticsModel::IsOnline(const FMessageAddress& ClientAddress) const
{
	INetworkMessagingExtension* Statistics = Private::GetMessagingStatistics();
	return Statistics && Statistics->GetNodeIdFromAddress(ClientAddress).IsValid();
}
