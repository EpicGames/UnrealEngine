// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientNetworkStatisticsModel.h"
#include "Misc/Optional.h"

namespace UE::MultiUserServer
{
	/** Synchronizes the network statistics (the statistics are updated async). */
	class FClientNetworkStatisticsModel : public IClientNetworkStatisticsModel
	{
	public:

		//~ Begin IClientNetworkStatisticsModel Interface
		virtual TOptional<FMessageTransportStatistics> GetLatestNetworkStatistics(const FMessageAddress& ClientAddress) const override;
		virtual bool IsOnline(const FMessageAddress& ClientAddress) const override;
		//~ End IClientNetworkStatisticsModel Interface
	};
}

