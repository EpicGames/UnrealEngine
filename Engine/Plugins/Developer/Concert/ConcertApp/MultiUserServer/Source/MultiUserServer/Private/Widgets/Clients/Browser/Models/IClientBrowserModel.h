// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

template <typename OptionalType> struct TOptional;

struct FConcertSessionClientInfo;
struct FConcertSessionInfo;
struct FMessageAddress;

namespace UE::MultiUserServer
{
	class FClientBrowserItem;
	
	/** Decouples the UI from the server functions. */
	class IClientBrowserModel
	{
	public:

		enum class EClientUpdateType : uint8
		{
			Added,
			Removed
		};
		
		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClientListChanged, TSharedPtr<FClientBrowserItem>, EClientUpdateType);
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnSessionListChanged, const FGuid& /*SessionId*/);

		/** Gets the IDs of all availabel IDs */
		virtual TSet<FGuid> GetSessions() const = 0;
		/** Gets more info about a session returned by GetSessions */
		virtual TOptional<FConcertSessionInfo> GetSessionInfo(const FGuid& SessionID) const = 0;

		virtual const TArray<TSharedPtr<FClientBrowserItem>>& GetItems() const = 0;

		virtual void SetKeepClientsAfterDisconnect(bool bNewValue) = 0;
		virtual bool ShouldKeepClientsAfterDisconnect() const = 0;
		
		/** Called when GeItems() changes its content */
		virtual FOnClientListChanged& OnClientListChanged() = 0;

		virtual FOnSessionListChanged& OnSessionCreated() = 0;
		virtual FOnSessionListChanged& OnSessionDestroyed() = 0;
		
		virtual ~IClientBrowserModel() = default;
	};
}


