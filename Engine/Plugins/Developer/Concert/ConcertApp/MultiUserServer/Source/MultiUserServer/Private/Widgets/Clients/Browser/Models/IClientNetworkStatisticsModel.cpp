// Copyright Epic Games, Inc. All Rights Reserved.

#include "IClientNetworkStatisticsModel.h"

#include "INetworkMessagingExtension.h"
#include "Math/UnitConversion.h"

namespace UE::MultiUserServer::NetworkStatistics
{
	FString FormatIPv4AsString(const TOptional<FMessageTransportStatistics>& Stats)
	{
		const FString Fallback(TEXT("No IP available"));
		if (Stats)
		{
			const FString DisplayString = Stats->IPv4AsString.IsEmpty()
				? Fallback
				: Stats->IPv4AsString;
			return DisplayString;
		}
		return Fallback;
	}
		
	FString FormatTotalBytesSent(const FMessageTransportStatistics& Stats)
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(Stats.TotalBytesSent, EUnit::Bytes);
		return FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));
	}
	
	FString FormatTotalBytesReceived(const FMessageTransportStatistics& Stats)
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(Stats.TotalBytesReceived, EUnit::Bytes);
		return FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));
	}
	
	FString FormatAverageRTT(const FMessageTransportStatistics& Stats)
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(static_cast<uint64>(Stats.AverageRTT.GetTotalMilliseconds()), EUnit::Milliseconds);
		return FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));;
	}
	
	FString FormatBytesInflight(const FMessageTransportStatistics& Stats)
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(Stats.BytesInflight, EUnit::Bytes);
		return FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));
	}
	
	FString FormatWindowSize(const FMessageTransportStatistics& Stats)
	{
		return FString::Printf(TEXT("%d"), Stats.WindowSize);
	}
	
	FString FormatTotalBytesLost(const FMessageTransportStatistics& Stats)
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(Stats.TotalBytesLost, EUnit::Bytes);
		return FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));
	}
}
