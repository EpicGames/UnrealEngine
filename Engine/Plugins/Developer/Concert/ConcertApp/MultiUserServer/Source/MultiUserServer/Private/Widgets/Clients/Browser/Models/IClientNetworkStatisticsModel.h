// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FString;
template <typename OptionalType> struct TOptional;

struct FConcertSessionClientInfo;
struct FConcertSessionInfo;
struct FMessageAddress;
struct FMessageTransportStatistics;
struct FTransferStatistics;

namespace UE::MultiUserServer
{
	/** Decouples the UI from the server functions. */
	class IClientNetworkStatisticsModel
	{
	public:
		
		virtual TOptional<FMessageTransportStatistics> GetLatestNetworkStatistics(const FMessageAddress& ClientAddress) const = 0;
		/** Whether this client is currently reachable */
		virtual bool IsOnline(const FMessageAddress& ClientAddress) const = 0;

		virtual ~IClientNetworkStatisticsModel() = default;
	};

	// Should move IClientNetworkStatisticsModel into the this namespace for more cohesion
	namespace NetworkStatistics
	{
		FString FormatIPv4AsString(const TOptional<FMessageTransportStatistics>& Stats);
		
		FString FormatTotalBytesSent(const FMessageTransportStatistics& Stats);
		FString FormatTotalBytesReceived(const FMessageTransportStatistics& Stats);
		FString FormatAverageRTT(const FMessageTransportStatistics& Stats);
		FString FormatBytesInflight(const FMessageTransportStatistics& Stats);
		FString FormatWindowSize(const FMessageTransportStatistics& Stats);		
		FString FormatTotalBytesLost(const FMessageTransportStatistics& Stats);
	}
}


