// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/DateTime.h"

struct FInboundTransferStatistics;
struct FOutboundTransferStatistics;

struct FMessageAddress;

namespace UE::MultiUserServer
{
	enum class EConcertTransferStatistic
	{
		SentToClient,
		ReceivedFromClient,

		Count
	};

	DECLARE_MULTICAST_DELEGATE(FOnTransferTimelineUpdated);
	DECLARE_MULTICAST_DELEGATE(FOnTransferGroupsUpdated);
	
	struct FConcertTransferSamplePoint
	{
		/** The time these stats were updated */
		FDateTime LocalTime;

		/** The number of bytes transferred, e.g. sent, received, etc. */
		uint64 BytesTransferred;

		FConcertTransferSamplePoint() = default;
		explicit FConcertTransferSamplePoint(const FDateTime& Time, uint64 BytesTransferred = 0)
			: LocalTime(Time)
			, BytesTransferred(BytesTransferred)
		{}
	};
	
	/** Keeps track of a single client's transfer statistics */
	class ITransferStatisticsModel
	{
	public:

		/** @return Contains the transfer statistics over time. Used for graphs. */
		virtual const TArray<FConcertTransferSamplePoint>& GetTransferStatTimeline(EConcertTransferStatistic StatisticType) const = 0;
		/** @return Sorted descending by message ID. Every message ID is unique. Used for list views. */
		virtual const TArray<TSharedPtr<FOutboundTransferStatistics>>& GetOutboundTransferStatsGroupedById() const = 0;
		/** @return Sorted descending by message ID. Every message ID is unique. Used for list views. */
		virtual const TArray<TSharedPtr<FInboundTransferStatistics>>& GetInboundTransferStatsGroupedById() const = 0;

		/** Called when GetTransferStatTimeline changes */
		virtual FOnTransferTimelineUpdated& OnTransferTimelineUpdated(EConcertTransferStatistic StatisticType) = 0;
		/** Called when GetTransferStatsGroupedById changes */
		virtual FOnTransferGroupsUpdated& OnOutboundTransferGroupsUpdated() = 0;
		/** Called when GetTransferStatsGroupedById changes */
		virtual FOnTransferGroupsUpdated& OnInboundTransferGroupsUpdated() = 0;

		virtual ~ITransferStatisticsModel() = default;
	};
}


