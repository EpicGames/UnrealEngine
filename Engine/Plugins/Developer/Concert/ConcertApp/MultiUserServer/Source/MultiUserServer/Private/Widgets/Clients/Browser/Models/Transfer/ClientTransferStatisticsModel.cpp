// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientTransferStatisticsModel.h"

#include "Features/IModularFeatures.h"
#include "Misc/ScopeExit.h"
#include "Widgets/Clients/Browser/Models/Transfer/TransferStatisticsModel.h"

namespace UE::MultiUserServer
{
	namespace Private::ClientTransferStatisticsModel
	{
		static INetworkMessagingExtension* GetMessagingStatistics()
		{
			IModularFeatures& ModularFeatures = IModularFeatures::Get();
			if (IsInGameThread())
			{
				if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
				{
					return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
				}
			}
			else
			{
				ModularFeatures.LockModularFeatureList();
				ON_SCOPE_EXIT
				{
					ModularFeatures.UnlockModularFeatureList();
				};
			
				if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
				{
					return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
				}
			}
		
			ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
			return nullptr;
		}
	}
	
	FClientTransferStatisticsModel::FClientTransferStatisticsModel(const FMessageAddress& ClientAddress)
		: FTransferStatisticsModelBase(
			TClientTransferStatTracker<FOutboundTransferStatistics>::FShouldInclude::CreateStatic(&FClientTransferStatisticsModel::ShouldIncludeOutboundStat, ClientAddress),
			TClientTransferStatTracker<FInboundTransferStatistics>::FShouldInclude::CreateStatic(&FClientTransferStatisticsModel::ShouldIncludeInboundStat, ClientAddress)
			)
	{}

	bool FClientTransferStatisticsModel::ShouldIncludeOutboundStat(const FOutboundTransferStatistics& Item, const FMessageAddress ClientAddress)
	{
		if (const INetworkMessagingExtension* Statistics = Private::ClientTransferStatisticsModel::GetMessagingStatistics())
		{
			const FGuid ClientNodeId = Statistics->GetNodeIdFromAddress(ClientAddress);
			return ClientNodeId == Item.DestinationId;
		}
		return false;
	}

	bool FClientTransferStatisticsModel::ShouldIncludeInboundStat(const FInboundTransferStatistics& Item, const FMessageAddress ClientAddress)
	{
		if (const INetworkMessagingExtension* Statistics = Private::ClientTransferStatisticsModel::GetMessagingStatistics())
		{
			const FGuid ClientNodeId = Statistics->GetNodeIdFromAddress(ClientAddress);
			return ClientNodeId == Item.OriginId;
		}
		return false; 
	}
}
