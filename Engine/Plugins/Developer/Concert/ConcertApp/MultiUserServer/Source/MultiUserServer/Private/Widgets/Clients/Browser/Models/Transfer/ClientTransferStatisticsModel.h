// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "TransferStatisticsModel.h"

namespace UE::MultiUserServer
{
	class FClientTransferStatisticsModel : public FTransferStatisticsModelBase
	{
	public:
		
		FClientTransferStatisticsModel(const FMessageAddress& ClientAddress);

	protected:
		
		static bool ShouldIncludeOutboundStat(const FOutboundTransferStatistics& Item, const FMessageAddress ClientAddress);
		static bool ShouldIncludeInboundStat(const FInboundTransferStatistics& Item, const FMessageAddress ClientAddress);
	};
}

