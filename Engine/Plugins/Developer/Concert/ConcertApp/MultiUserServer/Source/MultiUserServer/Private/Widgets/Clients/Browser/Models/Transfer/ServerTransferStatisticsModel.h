// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TransferStatisticsModel.h"

namespace UE::MultiUserServer
{
	class FServerTransferStatisticsModel : public FTransferStatisticsModelBase
	{
	public:

		FServerTransferStatisticsModel()
			:
			FTransferStatisticsModelBase(
			TClientTransferStatTracker<FOutboundTransferStatistics>::FShouldInclude::CreateLambda([](const auto&){ return true; }),
			TClientTransferStatTracker<FInboundTransferStatistics>::FShouldInclude::CreateLambda([](const auto& ){ return true; })
			)
		{}
	};
}

