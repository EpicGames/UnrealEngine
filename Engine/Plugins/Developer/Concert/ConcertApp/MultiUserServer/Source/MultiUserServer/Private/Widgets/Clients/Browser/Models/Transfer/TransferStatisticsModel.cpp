// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransferStatisticsModel.h"

#include "Features/IModularFeatures.h"
#include "Misc/ScopeExit.h"

namespace UE::MultiUserServer
{
	namespace Private::TransferStatisticsModel
	{
		static INetworkMessagingExtension* GetMessagingStatistics()
		{
			IModularFeatures& ModularFeatures = IModularFeatures::Get();
			if (IsInGameThread())
			{
				if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
				{
					return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
				}
			}
			else
			{
				ModularFeatures.LockModularFeatureList();
				ON_SCOPE_EXIT
				{
					ModularFeatures.UnlockModularFeatureList();
				};
			
				if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
				{
					return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
				}
			}
		
			ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
			return nullptr;
		}

		static TSharedRef<TClientTransferStatTracker<FOutboundTransferStatistics>> MakeOutbound(TClientTransferStatTracker<FOutboundTransferStatistics>::FShouldInclude ShouldIncludeOutboundFunc)
		{
			return MakeShared<TClientTransferStatTracker<FOutboundTransferStatistics>>(
				MoveTemp(ShouldIncludeOutboundFunc),
				TClientTransferStatTracker<FOutboundTransferStatistics>::FStatGetter::CreateLambda(
					[](const FOutboundTransferStatistics& Stats)
					{
						return Stats.BytesSent;
					}
				)
			);
		}

		static TSharedRef<TClientTransferStatTracker<FInboundTransferStatistics>> MakeInbound(TClientTransferStatTracker<FInboundTransferStatistics>::FShouldInclude ShouldIncludeInboundFunc)
		{
			return MakeShared<TClientTransferStatTracker<FInboundTransferStatistics>>(
				MoveTemp(ShouldIncludeInboundFunc),
				TClientTransferStatTracker<FInboundTransferStatistics>::FStatGetter::CreateLambda(
					[](const FInboundTransferStatistics& Stats)
					{
						return Stats.BytesReceived;
					}
				)
			);
		}
	}

	FTransferStatisticsModelBase::FTransferStatisticsModelBase(
		TClientTransferStatTracker<FOutboundTransferStatistics>::FShouldInclude ShouldIncludeOutboundFunc,
		TClientTransferStatTracker<FInboundTransferStatistics>::FShouldInclude ShouldIncludeInboundFunc)
		: OutboundStatTracker(Private::TransferStatisticsModel::MakeOutbound(MoveTemp(ShouldIncludeOutboundFunc)))
		, InboundStatTracker(Private::TransferStatisticsModel::MakeInbound(MoveTemp(ShouldIncludeInboundFunc)))
	{
		if (INetworkMessagingExtension* Statistics = Private::TransferStatisticsModel::GetMessagingStatistics())
		{
			Statistics->OnOutboundTransferUpdatedFromThread().AddSP(OutboundStatTracker, &TClientTransferStatTracker<FOutboundTransferStatistics>::OnTransferUpdatedFromThread);
			Statistics->OnInboundTransferUpdatedFromThread().AddSP(InboundStatTracker, &TClientTransferStatTracker<FInboundTransferStatistics>::OnTransferUpdatedFromThread);
			InboundTickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(OutboundStatTracker, &TClientTransferStatTracker<FOutboundTransferStatistics>::Tick));
			OutboundTickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(InboundStatTracker, &TClientTransferStatTracker<FInboundTransferStatistics>::Tick));
		}
	}

	FTransferStatisticsModelBase::~FTransferStatisticsModelBase()
	{
		if (INetworkMessagingExtension* Statistics = Private::TransferStatisticsModel::GetMessagingStatistics())
		{
			Statistics->OnOutboundTransferUpdatedFromThread().RemoveAll(&OutboundStatTracker);
			Statistics->OnInboundTransferUpdatedFromThread().RemoveAll(&InboundStatTracker);
		}
		
		FTSTicker::RemoveTicker(InboundTickHandle);
		FTSTicker::RemoveTicker(OutboundTickHandle);
	}

	const TArray<FConcertTransferSamplePoint>& FTransferStatisticsModelBase::GetTransferStatTimeline(EConcertTransferStatistic StatisticType) const
	{
		switch (StatisticType)
		{
		case EConcertTransferStatistic::SentToClient: return OutboundStatTracker->GetTransferStatisticsTimeline();
		case EConcertTransferStatistic::ReceivedFromClient: return InboundStatTracker->GetTransferStatisticsTimeline();;
		case EConcertTransferStatistic::Count:
		default:
			checkNoEntry();
			return OutboundStatTracker->GetTransferStatisticsTimeline();
		}
	}

	FOnTransferTimelineUpdated& FTransferStatisticsModelBase::OnTransferTimelineUpdated(EConcertTransferStatistic StatisticType)
	{
		switch (StatisticType)
		{
		case EConcertTransferStatistic::SentToClient: return OutboundStatTracker->GetOnTimelineUpdatedDelegates();
		case EConcertTransferStatistic::ReceivedFromClient: return InboundStatTracker->GetOnTimelineUpdatedDelegates();;
		case EConcertTransferStatistic::Count:
		default:
			checkNoEntry();
			return OutboundStatTracker->GetOnTimelineUpdatedDelegates();
		}
	}
}
