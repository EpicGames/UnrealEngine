// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SpscQueue.h"
#include "Containers/Ticker.h"
#include "INetworkMessagingExtension.h"
#include "Widgets/Clients/Browser/Models/ITransferStatisticsModel.h"

namespace UE::MultiUserServer
{
	template<typename TStatBase>
	class TClientTransferStatTracker
	{
		struct FIncompleteMessageData
		{
			uint64 BytesTransferredSoFar;
			FDateTime LastUpdate;
		};
		
		struct TStatItem : TStatBase
		{
			FDateTime LocalTime;

			TStatItem(const FDateTime& LocalTime, TStatBase Data)
				: TStatBase(Data)
				, LocalTime(LocalTime)
			{}
		};

		using FMessageId = decltype(FOutboundTransferStatistics::MessageId);
		
	public:

		DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldInclude, const TStatBase&);
		DECLARE_DELEGATE_RetVal_OneParam(uint64, FStatGetter, const TStatBase&);

		TClientTransferStatTracker(FShouldInclude ShouldIncludeFunc, FStatGetter StatGetterFunc)
			: ShouldIncludeFunc(MoveTemp(ShouldIncludeFunc))
			, StatGetterFunc(MoveTemp(StatGetterFunc))
		{}

		void OnTransferUpdatedFromThread(TStatBase TransferStatistics)
		{
			if (ShouldIncludeFunc.Execute(TransferStatistics))
			{
				AsyncStatQueue.Enqueue(TStatItem{ FDateTime::Now(), TransferStatistics });
			}
		}

		bool Tick(float DeltaTime)
		{
			const bool bRemovedOldStats = RemoveOldStatTimelines();
			const bool bRemovedAnyGroups = RemoveOldGroupedStats();
			const bool bUpdatedAnyTimeline = !AsyncStatQueue.IsEmpty();
			
			const FDateTime Now = FDateTime::Now();
			FConcertTransferSamplePoint SamplingThisTick{ Now };
			while (const TOptional<TStatItem> TransferStatistics = AsyncStatQueue.Dequeue())
			{
				UpdateStatTimeline(TransferStatistics.GetValue(), SamplingThisTick);
				UpdateGroupedStats(TransferStatistics.GetValue());
			}

			if (bRemovedOldStats || bUpdatedAnyTimeline || bRemovedAnyGroups)
			{
				TransferStatisticsTimeline.Add(SamplingThisTick);
				OnTimelineUpdatedDelegates.Broadcast();
				
				OnGroupsUpdatedDelegate.Broadcast();
			}

			return true;
		}
		
		FOnTransferTimelineUpdated& GetOnTimelineUpdatedDelegates() { return OnTimelineUpdatedDelegates; }
		FOnTransferGroupsUpdated& GetOnGroupsUpdatedDelegate() { return OnGroupsUpdatedDelegate; }
		const TArray<FConcertTransferSamplePoint>& GetTransferStatisticsTimeline() const { return TransferStatisticsTimeline; }
		const TArray<TSharedPtr<TStatBase>>& GetTransferStatisticsGroupedById() const { return TransferStatisticsGroupedById; }

	private:
		
		TSpscQueue<TStatItem> AsyncStatQueue;
		
		TArray<FConcertTransferSamplePoint> TransferStatisticsTimeline;
		TMap<FMessageId, FIncompleteMessageData> IncompleteStatsUntilNow;
		
		TArray<TSharedPtr<TStatBase>> TransferStatisticsGroupedById;
		TMap<FMessageId, FDateTime> LastUpdateGroupUpdates;
		
		FOnTransferTimelineUpdated OnTimelineUpdatedDelegates;
		FOnTransferGroupsUpdated OnGroupsUpdatedDelegate;

		FShouldInclude ShouldIncludeFunc;
		FStatGetter StatGetterFunc;
		
		static bool IsTooOld(const FDateTime& StatCreationTime, const FDateTime& Now)
		{
			const FTimespan RetainTime = FTimespan::FromSeconds(60.f);
			return RetainTime < Now - StatCreationTime;
		}
		
		bool RemoveOldStatTimelines()
		{
			bool bModifiedTimelines = false;
			
			const FDateTime Now = FDateTime::Now();
			for (auto SampleIt = TransferStatisticsTimeline.CreateIterator(); SampleIt; ++SampleIt)
			{
				if (IsTooOld(SampleIt->LocalTime, Now))
				{
					SampleIt.RemoveCurrent();
					bModifiedTimelines = true;
				}
			}

			for (auto IncompleteIt = IncompleteStatsUntilNow.CreateIterator(); IncompleteIt; ++IncompleteIt)
			{
				if (IsTooOld(IncompleteIt->Value.LastUpdate, Now))
				{
					IncompleteIt.RemoveCurrent();
				}
			}

			return bModifiedTimelines;
		}
		
		bool RemoveOldGroupedStats()
		{
			bool bRemovedAny = false;
			const FDateTime Now = FDateTime::Now();
			for (auto It = TransferStatisticsGroupedById.CreateIterator(); It; ++It)
			{
				const TSharedPtr<TStatBase>& Stat = *It;
				if (IsTooOld(LastUpdateGroupUpdates[Stat->MessageId], Now))
				{
					LastUpdateGroupUpdates.Remove(Stat->MessageId);
					It.RemoveCurrent();
					bRemovedAny = true;
				}
			}
			return bRemovedAny;
		}
		
		void UpdateStatTimeline(const TStatBase& TransferStatistics, FConcertTransferSamplePoint& SamplingThisTick)
		{
			uint64 BytesSentSinceLastTime = StatGetterFunc.Execute(TransferStatistics);
			// Same message ID may have been queued multiple times
			if (FIncompleteMessageData* ItemThisTick = IncompleteStatsUntilNow.Find(TransferStatistics.MessageId))
			{
				BytesSentSinceLastTime -= ItemThisTick->BytesTransferredSoFar;
			}

			IncompleteStatsUntilNow.Add(TransferStatistics.MessageId, { StatGetterFunc.Execute(TransferStatistics), FDateTime::Now() });
			SamplingThisTick.BytesTransferred += BytesSentSinceLastTime;
		}
		
		void UpdateGroupedStats(const TStatBase& TransferStatistics)
		{
			const TSharedPtr<TStatBase> NewValue = MakeShared<TStatBase>(TransferStatistics);
			const auto Pos = Algo::LowerBound(TransferStatisticsGroupedById, NewValue, [](const TSharedPtr<TStatBase>& Value, const TSharedPtr<TStatBase>& Check)
			{
				return Value->MessageId < Check->MessageId;
			});
			if (TransferStatisticsGroupedById.Num() > 0 && Pos < TransferStatisticsGroupedById.Num() && TransferStatisticsGroupedById[Pos] && TransferStatisticsGroupedById[Pos]->MessageId == NewValue->MessageId)
			{
				*TransferStatisticsGroupedById[Pos] = *NewValue;
			}
			else
			{
				TransferStatisticsGroupedById.Insert(NewValue, Pos);
			}

			LastUpdateGroupUpdates.Add(NewValue->MessageId, FDateTime::Now());
		}
	};

	class FTransferStatisticsModelBase : public ITransferStatisticsModel
	{
	public:
		
		FTransferStatisticsModelBase(
			TClientTransferStatTracker<FOutboundTransferStatistics>::FShouldInclude ShouldIncludeOutboundFunc,
			TClientTransferStatTracker<FInboundTransferStatistics>::FShouldInclude ShouldIncludeInboundFunc
			);
		virtual ~FTransferStatisticsModelBase() override;
		
		virtual const TArray<FConcertTransferSamplePoint>& GetTransferStatTimeline(EConcertTransferStatistic StatisticType) const override;
		virtual const TArray<TSharedPtr<FOutboundTransferStatistics>>& GetOutboundTransferStatsGroupedById() const override { return OutboundStatTracker->GetTransferStatisticsGroupedById(); }
		virtual const TArray<TSharedPtr<FInboundTransferStatistics>>& GetInboundTransferStatsGroupedById() const override { return InboundStatTracker->GetTransferStatisticsGroupedById(); }
		virtual FOnTransferTimelineUpdated& OnTransferTimelineUpdated(EConcertTransferStatistic StatisticType) override;
		virtual FOnTransferGroupsUpdated& OnOutboundTransferGroupsUpdated() override { return OutboundStatTracker->GetOnGroupsUpdatedDelegate(); }
		virtual FOnTransferGroupsUpdated& OnInboundTransferGroupsUpdated() override { return InboundStatTracker->GetOnGroupsUpdatedDelegate(); }

	private:

		FTSTicker::FDelegateHandle InboundTickHandle;
		FTSTicker::FDelegateHandle OutboundTickHandle;
		

		// These are used for subscribing to INetworkMessagingExtension - therefore they are shared pointers for synchronization purposes
		// Prevents race condition when destructor is called and UDP thread attempts to call the callbacks
		TSharedRef<TClientTransferStatTracker<FOutboundTransferStatistics>> OutboundStatTracker;
		TSharedRef<TClientTransferStatTracker<FInboundTransferStatistics>> InboundStatTracker;
	};
}

