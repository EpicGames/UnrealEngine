// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClientNetworkStats.h"


#include "Widgets/Clients/Browser/Models/IClientNetworkStatisticsModel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SClientNetworkStats"

namespace UE::MultiUserServer
{
	void SClientNetworkStats::Construct(const FArguments& InArgs)
	{
		NetworkStatistics = InArgs._NetworkStatistics;
		
		HighlightText = InArgs._HighlightText;
		check(HighlightText);
		
		ChildSlot
		.HAlign(HAlign_Fill)
		[
			CreateStatTable()
		];
	}

	void SClientNetworkStats::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime,const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		if (const TOptional<FMessageTransportStatistics> Stats = NetworkStatistics.Get())
		{
			UpdateStatistics(*Stats);
		}
	}

	void SClientNetworkStats::UpdateStatistics(const FMessageTransportStatistics& Statistics) const
	{
		SendText->SetText(FText::FromString(NetworkStatistics::FormatTotalBytesSent(Statistics)));
		ReceiveText->SetText(FText::FromString(NetworkStatistics::FormatTotalBytesReceived(Statistics)));
		RoundTripTimeText->SetText(FText::FromString(NetworkStatistics::FormatAverageRTT(Statistics)));
		InflightText->SetText(FText::FromString(NetworkStatistics::FormatBytesInflight(Statistics)));
		WindowSize->SetText(FText::FromString(NetworkStatistics::FormatWindowSize(Statistics)));
		LossText->SetText(FText::FromString(NetworkStatistics::FormatTotalBytesLost(Statistics)));
	}

	TSharedRef<SWidget> SClientNetworkStats::CreateStatTable()
	{
		const TSharedRef<SHorizontalBox> Content = SNew(SHorizontalBox)
			.ToolTipText_Lambda([this]()
			{
				const TOptional<FMessageTransportStatistics> Stats = NetworkStatistics.Get();
				return Stats.IsSet() ? FText::GetEmpty() : LOCTEXT("ErrorGettingStats", "Error getting stats");
			});
		
		AddStatistic(Content, LOCTEXT("SentLabel", "Sent"), LOCTEXT("SentTooltip", "Total bytes sent to this client"), SendText);
		AddStatistic(Content, LOCTEXT("ReceiveLabel", "Received"), LOCTEXT("ReceiveTooltip", "Total bytes received from this client"), ReceiveText);
		AddStatistic(Content, LOCTEXT("RttLabel", "RTT"), LOCTEXT("RttTooltip", "Round trip time"), RoundTripTimeText);
		AddStatistic(Content, LOCTEXT("InflightLabel", "Inflight"), LOCTEXT("InflightTooltip", "Total reliable bytes awaiting an ack from client"), InflightText);
		AddStatistic(Content, LOCTEXT("WindowLabel", "WSize"), LOCTEXT("WindowSizeTooltip", "The window size for the end point. This is how big of a window we utilize for sending data. Bigger values are better. Small values indicate poor performance."), WindowSize);
		// Loss should be at the right
		SHorizontalBox::FScopedWidgetSlotArguments LossSlat = Content->AddSlot();
		AddStatistic(LossSlat, LOCTEXT("LossLabel", "Lost"), LOCTEXT("LostTooltip", "Total bytes lost while sending to the client"), LossText);
		LossSlat.HAlign(HAlign_Right);
		LossSlat.FillWidth(1.f);
		
		return Content;
	}
	
	void SClientNetworkStats::AddStatistic(const TSharedRef<SHorizontalBox>& AddTo, const FText& StatisticName, const FText& StatisticToolTip, TSharedPtr<STextBlock>& AssignTo)
	{
		SHorizontalBox::FScopedWidgetSlotArguments Slot = AddTo->AddSlot();
		AddStatistic(Slot, StatisticName, StatisticToolTip, AssignTo);
	}

	void SClientNetworkStats::AddStatistic(SHorizontalBox::FScopedWidgetSlotArguments& Slot, const FText& StatisticName, const FText& StatisticToolTip, TSharedPtr<STextBlock>& AssignTo)
	{
		Slot
		.AutoWidth()
		.Padding(3.f)
		[
			SNew(SVerticalBox)
			.ToolTipText(StatisticToolTip)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(StatisticName)
				.ColorAndOpacity(FColor::White)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(AssignTo, STextBlock)
				.ColorAndOpacity(FColor::White)
				.HighlightText_Lambda([this](){ return *HighlightText; })
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE
