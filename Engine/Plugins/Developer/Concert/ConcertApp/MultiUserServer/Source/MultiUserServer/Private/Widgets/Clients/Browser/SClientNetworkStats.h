// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INetworkMessagingExtension.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

class STextBlock;

namespace UE::MultiUserServer
{
	/**
	 * Displays statistics about a client connection in a table like format: send, receive, RTT, Inflight, and Loss.
	 */
	class SClientNetworkStats : public SCompoundWidget
	{
	public:
		
		SLATE_BEGIN_ARGS(SClientNetworkStats){}
			SLATE_ARGUMENT(TSharedPtr<FText>, HighlightText)
			SLATE_ATTRIBUTE(TOptional<FMessageTransportStatistics>, NetworkStatistics)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		//~ Begin SCompoundWidget Interface		
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		//~ End SCompoundWidget Interface

	private:

		TAttribute<TOptional<FMessageTransportStatistics>> NetworkStatistics;
		
		/** The text to highlight */
		TSharedPtr<FText> HighlightText;

		TSharedPtr<STextBlock> SendText;
		TSharedPtr<STextBlock> ReceiveText;
		TSharedPtr<STextBlock> RoundTripTimeText;
		TSharedPtr<STextBlock> InflightText;
		TSharedPtr<STextBlock> LossText;
		TSharedPtr<STextBlock> WindowSize;
		
		void UpdateStatistics(const FMessageTransportStatistics& Statistics) const;

		TSharedRef<SWidget> CreateStatTable();
		void AddStatistic(const TSharedRef<SHorizontalBox>& AddTo, const FText& StatisticName, const FText& StatisticToolTip, TSharedPtr<STextBlock>& AssignTo);
		void AddStatistic(SHorizontalBox::FScopedWidgetSlotArguments& Slot, const FText& StatisticName, const FText& StatisticToolTip, TSharedPtr<STextBlock>& AssignTo);
	};
}

