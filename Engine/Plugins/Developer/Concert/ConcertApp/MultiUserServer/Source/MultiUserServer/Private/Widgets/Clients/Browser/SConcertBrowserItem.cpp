// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertBrowserItem.h"

#include "ConcertServerStyle.h"
#include "SClientNetworkStats.h"
#include "Graph/SClientNetworkGraphs.h"
#include "Models/IClientNetworkStatisticsModel.h"
#include "Table/SClientOutboundTransferStatTable.h"
#include "Table/SClientInboundTransferStatTable.h"
#include "Widgets/Clients/Browser/Item/IConcertBrowserItem.h"

#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertClientBrowserItem"

void UE::MultiUserServer::SConcertBrowserItem::Construct(const FArguments& InArgs, TSharedRef<IConcertBrowserItem> InClientItem)
{
	Item = MoveTemp(InClientItem);
	HighlightText = InArgs._HighlightText;

	ChildSlot
	.Padding(FMargin(0.0f, 0.0f, 4.0f, 4.0f))
	[
		// Shadow behind thumbnail
		SNew(SBorder)
		.BorderImage(FConcertServerStyle::Get().GetBrush("Concert.Clients.DropShadow"))
		.Padding(FMargin(0.0f, 0.0f, 5.0f, 5.0f))
		[
			// Change outside of thumbnail depending on hover state; lighter than inside
			SNew(SBorder)
			.BorderImage(this, &SConcertBrowserItem::GetBackgroundImage)
			.Padding(2.f)
			[
				// Inside of thumbnail is darker
				SNew(SBorder)
				.BorderImage(FConcertServerStyle::Get().GetBrush("Concert.Clients.ThumbnailTitle"))
				.Padding(2.f)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Top)
					[
						CreateHeader()	
					]
					
					+SVerticalBox::Slot()
					.FillHeight(1.f)
					.Padding(-2.f, 5.f, -2.f, 0.f)
					[
						CreateContentArea()
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, 5.f, 0.f, 0.f)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Bottom)
					[
						CreateStats()
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(-2.f, 5.f, -2.f, -2.f)
					.VAlign(VAlign_Bottom)
					[
						CreateFooter()
					]
				]
			]
		]
	];
}

void UE::MultiUserServer::SConcertBrowserItem::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (const TOptional<FMessageTransportStatistics> Stats = Item->GetLatestNetworkStatistics())
	{
		LastAvailableIp = NetworkStatistics::FormatIPv4AsString(Stats);
	}
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertBrowserItem::CreateHeader()
{
	return SAssignNew(ClientName, STextBlock)
		.Font(FConcertServerStyle::Get().GetFontStyle("Concert.Clients.ClientNameTileFont"))
		.Text_Lambda([this](){ return FText::FromString(Item->GetDisplayName()); })
		.ToolTipText_Lambda([this](){ return Item->GetToolTip(); })
		.HighlightText_Lambda([this](){ return *HighlightText; })
		.ColorAndOpacity(FColor::White);
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertBrowserItem::CreateContentArea()
{
	return SNew(SOverlay)
		+SOverlay::Slot()
		[
			SNew(SClientNetworkGraphs, Item->GetTransferStatistics())
			.Visibility_Lambda([this](){ return Item->GetDisplayMode() == EConcertBrowserItemDisplayMode::NetworkGraph ? EVisibility::Visible : EVisibility::Collapsed; })
		]
		+SOverlay::Slot().Padding(4.f, 0.f)
		[
			SNew(SClientOutboundTransferStatTable, Item->GetTransferStatistics())
			.Visibility_Lambda([this](){ return Item->GetDisplayMode() == EConcertBrowserItemDisplayMode::OutboundSegementTable ? EVisibility::Visible : EVisibility::Collapsed; })
		]
		+SOverlay::Slot().Padding(4.f, 0.f)
		[
			SNew(SClientInboundTransferStatTable, Item->GetTransferStatistics())
			.Visibility_Lambda([this](){ return Item->GetDisplayMode() == EConcertBrowserItemDisplayMode::InboundSegmentTable ? EVisibility::Visible : EVisibility::Collapsed; })
		];
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertBrowserItem::CreateStats()
{
	return SNew(SScaleBox)
		.Stretch(EStretch::ScaleToFit)
		.HAlign(HAlign_Fill)
		[
			SAssignNew(NetworkStats, SClientNetworkStats)
			.HighlightText(HighlightText)
			.NetworkStatistics_Lambda([this](){ return Item->GetLatestNetworkStatistics(); })
		];
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertBrowserItem::CreateFooter()
{
	return SNew(SBorder)
		.BorderImage(FConcertServerStyle::Get().GetBrush("Concert.Clients.ThumbnailFooter"))
		[
			SNew(SHorizontalBox)

			// Online / offline indicator
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(2.f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.FilledCircle"))
				.ColorAndOpacity_Lambda([this]()
				{
					return Item->IsOnline()
						? FStyleColors::AccentGreen
						: FStyleColors::AccentGray;
				})
				.ToolTipText_Lambda([this]()
				{
					return Item->IsOnline()
						? LOCTEXT("ConnectionIndicator.Online", "Connected")
						: LOCTEXT("ConnectionIndicator.Offline", "Not reachable");
				})
			]

			// IP address
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.Padding(2.f)
			[
				SAssignNew(ClientIP4, STextBlock)
				.ColorAndOpacity(FColor::White)
				.HighlightText_Lambda([this](){ return *HighlightText; })
				.Text(this, &SConcertBrowserItem::GetIpText)
			]
		];
}

FText UE::MultiUserServer::SConcertBrowserItem::GetIpText() const
{
	if (LastAvailableIp)
	{
		return FText::FromString(*LastAvailableIp);
	}
	return FText::FromString(NetworkStatistics::FormatIPv4AsString({}));
}

const FSlateBrush* UE::MultiUserServer::SConcertBrowserItem::GetBackgroundImage() const
{
	if (IsHovered())
	{
		return FConcertServerStyle::Get().GetBrush("Concert.Clients.ThumbnailAreaHoverBackground");
	}
	return FConcertServerStyle::Get().GetBrush("Concert.Clients.ThumbnailAreaBackground");
}

#undef LOCTEXT_NAMESPACE
