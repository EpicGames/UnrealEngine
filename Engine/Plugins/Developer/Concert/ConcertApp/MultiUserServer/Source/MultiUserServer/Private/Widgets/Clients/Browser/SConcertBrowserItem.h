// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class STextBlock;

namespace UE::MultiUserServer
{
	class SClientNetworkStats;
	class IClientNetworkStatisticsModel;
	class IConcertBrowserItem;
	
	class SConcertBrowserItem : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SConcertBrowserItem) {}
			SLATE_ARGUMENT(TSharedPtr<FText>, HighlightText)
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs, TSharedRef<IConcertBrowserItem> InClientItem);
		/** Updates LastAvailableIp */
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	private:

		/** What we're displaying */
		TSharedPtr<IConcertBrowserItem> Item;

		/** The text to highlight */
		TSharedPtr<FText> HighlightText;

		TSharedPtr<STextBlock> ClientName;
		TSharedPtr<SClientNetworkStats> NetworkStats;
		TSharedPtr<STextBlock> ClientIP4;

		/** Stores the IP address once it becomes it available. Used so we can continue displaying it when the client goes offline. */
		TOptional<FString> LastAvailableIp;
		
		TSharedRef<SWidget> CreateHeader();
		TSharedRef<SWidget> CreateContentArea();
		TSharedRef<SWidget> CreateStats();
		TSharedRef<SWidget> CreateFooter();

		FText GetIpText() const;

		const FSlateBrush* GetBackgroundImage() const;
	};
}
