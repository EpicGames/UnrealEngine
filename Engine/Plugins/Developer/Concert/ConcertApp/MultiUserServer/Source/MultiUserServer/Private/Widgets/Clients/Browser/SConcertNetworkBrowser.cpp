// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertNetworkBrowser.h"

#include "ConcertMessageData.h"
#include "ConcertServerStyle.h"
#include "Dialog/SCustomDialog.h"
#include "MultiUserServerModule.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SConcertBrowserItem.h"
#include "Misc/TextFilter.h"
#include "Widgets/Clients/Browser/Item/ClientBrowserItem.h"

#include "Algo/AllOf.h"
#include "Algo/Count.h"
#include "Dialog/SMessageDialog.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Clients/Browser/Item/IConcertBrowserItem.h"
#include "Widgets/Clients/Browser/Models/IClientBrowserModel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Clients/Browser/Item/ServerBrowserItem.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STileView.h"
#include "Window/ModalWindowManager.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertClientBrowser"

void UE::MultiUserServer::SConcertNetworkBrowser::Construct(const FArguments& InArgs, TSharedRef<IClientBrowserModel> InBrowserModel, TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel, TSharedRef<IConcertServer> Server)
{
	BrowserModel = MoveTemp(InBrowserModel);

	ServerItem = MakeShared<FServerBrowserItem>(Server, NetworkStatisticsModel);
		
	HighlightText = MakeShared<FText>();
	SessionFilter = MakeShared<FBrowserItemTextFilter>(FBrowserItemTextFilter::FItemToStringArray::CreateSP(this, &SConcertNetworkBrowser::GenerateSearchTerms));
	SessionFilter->OnChanged().AddSP(this, &SConcertNetworkBrowser::UpdateTileViewFromAllowedSessions);

	OnServerDoubleClicked = InArgs._OnServerDoubleClicked;
	OnClientDoubleClicked = InArgs._OnClientDoubleClicked;

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(4.f)
		.AutoHeight()
		[
			CreateSearchArea(InArgs)
		]

		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(5.f)
		[
			SNew(SOverlay)

			+SOverlay::Slot()
			[
				CreateTileView()
			]

			+SOverlay::Slot()
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility_Lambda([this](){ return DisplayedItems.Num() == 0 ? EVisibility::Visible : EVisibility::Hidden; })
				[
					SNew(STextBlock)
					.Text(this, &SConcertNetworkBrowser::GetErrorMessageText)
				]
			]
		]
	];

	BrowserModel->OnSessionCreated().AddSP(this, &SConcertNetworkBrowser::OnSessionCreated);
	BrowserModel->OnSessionDestroyed().AddSP(this, &SConcertNetworkBrowser::OnSessionDestroyed);
	BrowserModel->OnClientListChanged().AddSP(this, &SConcertNetworkBrowser::OnClientListChanged);

	AllowAllSessions();
}

void UE::MultiUserServer::SConcertNetworkBrowser::ShowOnlyClientsFromSession(const FGuid& SessionId)
{
	AllowedSessions = { SessionId };
	UpdateTileViewFromAllowedSessions();
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertNetworkBrowser::CreateSearchArea(const FArguments& InArgs)
{
	return SNew(SHorizontalBox)
	
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &SConcertNetworkBrowser::CreateSessionFilterOptionMenu)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return AllowedSessions.Num() == BrowserModel->GetSessions().Num()
						? LOCTEXT("MessageActionFilter.Selection.All", "All")
						: FText::FromString(FString::FromInt(AllowedSessions.Num()));
				})
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(0.65f)
		.Padding(4.f, 0.f)
		.VAlign(VAlign_Fill)
		[
			SNew(SSearchBox)
			.OnTextChanged_Lambda([this](const FText& SearchText)
			{
				*HighlightText = SearchText;
				SessionFilter->SetRawFilterText(SearchText);
			})
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			InArgs._RightOfSearch.Widget
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			CreateOptionsButton()
		];
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertNetworkBrowser::CreateOptionsButton()
{
	return SNew(SComboButton)
			.ComboButtonStyle( FAppStyle::Get(), "SimpleComboButtonWithIcon" )
			.OnGetMenuContent( this, &SConcertNetworkBrowser::CreateOptionsButtonMenu)
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image( FAppStyle::Get().GetBrush("Icons.Settings") )
			];
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertNetworkBrowser::CreateTileView()
{
	constexpr int32 Height = 270;
	constexpr int32 Width = 270;
	return SAssignNew(TileView, STileView<TSharedPtr<IConcertBrowserItem>>)
		.SelectionMode(ESelectionMode::Multi)
		.ListItemsSource(&DisplayedItems)
		.OnGenerateTile(this, &SConcertNetworkBrowser::MakeTileViewWidget)
		.OnContextMenuOpening(this, &SConcertNetworkBrowser::OnGetContextMenuContent)
		.OnMouseButtonDoubleClick(this, &SConcertNetworkBrowser::OnListMouseButtonDoubleClick)
		.ItemHeight(Height)
		.ItemWidth(Width)
	;
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertNetworkBrowser::CreateOptionsButtonMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("KeepDisconnectedClients.Label", "Keep Disconnected"),
		LOCTEXT("KeepDisconnectedClients.Tooltip", "Whether to keep clients that have disconnected in memory. This may be useful in unstable networks when you want to analyse why clients keep disconnecting."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this](){ ToggleShouldKeepDisconnected(); }),
			FCanExecuteAction::CreateLambda([](){ return true; }),
			FIsActionChecked::CreateLambda([this](){ return BrowserModel->ShouldKeepClientsAfterDisconnect(); })
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	
	return MenuBuilder.MakeWidget();
}

void UE::MultiUserServer::SConcertNetworkBrowser::ToggleShouldKeepDisconnected()
{
	const bool bKeepDisconnected = !BrowserModel->ShouldKeepClientsAfterDisconnect();
	if (bKeepDisconnected)
	{
		BrowserModel->SetKeepClientsAfterDisconnect(true);
		return;
	}
				
	const int32 NumDisconnected = Algo::CountIf(BrowserModel->GetItems(), [](const TSharedPtr<FClientBrowserItem>& Item) { return Item->IsDisconnected(); });
	if (NumDisconnected == 0)
	{
		BrowserModel->SetKeepClientsAfterDisconnect(false);
		return;
	}
				
	const TSharedRef<SMessageDialog> Dialog = SNew(SMessageDialog)
		.Title(LOCTEXT("RemoveDisconnectedClients.Title", "Remove disconnected clients?"))
		.Icon(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
		.Message(FText::Format(LOCTEXT("RemoveDisconnectedClients.MessageFmt", "There are {0} disconnected clients. If you proceed, these clients will be removed from the session browser; opened log tabs will remain open.\nProceed?"), NumDisconnected))
		.UseScrollBox(false)
		.Buttons({
			SMessageDialog::FButton(LOCTEXT("RemoveButton", "Remove"))
				.SetOnClicked(FSimpleDelegate::CreateLambda([this]()
				{
					BrowserModel->SetKeepClientsAfterDisconnect(false);
				})),
			SMessageDialog::FButton(LOCTEXT("CancelButton", "Keep"))
				.SetPrimary(true)
				.SetFocus()
		});
	FConcertServerUIModule::Get().GetModalWindowManager()->ShowFakeModalWindow(Dialog);
}

void UE::MultiUserServer::SConcertNetworkBrowser::OnSessionCreated(const FGuid& SessionId)
{
	if (bShowAllSessions)
	{
		AllowedSessions.Add(SessionId);
		UpdateTileViewFromAllowedSessions();
	}
}

void UE::MultiUserServer::SConcertNetworkBrowser::OnSessionDestroyed(const FGuid& SessionId)
{
	if (AllowedSessions.Contains(SessionId))
	{
		AllowedSessions.Remove(SessionId);
		UpdateTileViewFromAllowedSessions();
	}
}

void UE::MultiUserServer::SConcertNetworkBrowser::OnClientListChanged(TSharedPtr<FClientBrowserItem> Item, IClientBrowserModel::EClientUpdateType UpdateType)
{
	switch (UpdateType)
	{
	case IClientBrowserModel::EClientUpdateType::Added:
		if (PassesFilter(Item))
		{
			DisplayedClientItems.Add(Item);
			DisplayedItems.Add(Item);
		}
		break;
	case IClientBrowserModel::EClientUpdateType::Removed:
		DisplayedClientItems.Remove(Item);
		DisplayedItems.Remove(Item);
		break;
	default:
		checkNoEntry();
	}

	TileView->RequestListRefresh();
}

TSharedRef<SWidget> UE::MultiUserServer::SConcertNetworkBrowser::SConcertNetworkBrowser::CreateSessionFilterOptionMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SelectAll", "All"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				if (BrowserModel->GetSessions().Num() == AllowedSessions.Num())
				{
					DisallowAllSessions();
				}
				else
				{
					AllowAllSessions();
				}
			}),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this](){ return BrowserModel->GetSessions().Num() == AllowedSessions.Num(); })),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SessionlessEndpoints.Label", "Show Sessionless clients"),
		LOCTEXT("SessionlessEndpoints.Tooltip", "Whether to show clients that are only discovering available sessions"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				bShowSessionlessClients = !bShowSessionlessClients;
				UpdateTileViewFromAllowedSessions();
			}),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this](){ return bShowSessionlessClients; })),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	MenuBuilder.AddSeparator();
	for (const FGuid& SessionId : BrowserModel->GetSessions())
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString(BrowserModel->GetSessionInfo(SessionId)->SessionName),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, SessionId]()
				{
					if (AllowedSessions.Contains(SessionId))
					{
						AllowedSessions.Remove(SessionId);
					}
					else
					{
						AllowedSessions.Add(SessionId);
					}
					UpdateTileViewFromAllowedSessions();
				}),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this, SessionId]{ return AllowedSessions.Contains(SessionId); })),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}
	return MenuBuilder.MakeWidget();
}

FText UE::MultiUserServer::SConcertNetworkBrowser::GetErrorMessageText() const
{
	if (BrowserModel->GetSessions().Num() == 0)
	{
		return LOCTEXT("NoLiveSessions", "No live sessions");
	}
	
	if (AllowedSessions.Num() == 0)
	{
		return LOCTEXT("NoSessionsSelected", "All live sessions filtered out");
	}

	const bool bAtLeastOneClient = BrowserModel->GetItems().Num() > 0;
	if (bAtLeastOneClient)
	{
		return LOCTEXT("AllFilteredOut", "All results have been filtered. Try changing your active filters above.");
	}
	
	return LOCTEXT("NoClients", "No known clients");
}

TSharedRef<ITableRow> UE::MultiUserServer::SConcertNetworkBrowser::MakeTileViewWidget(TSharedPtr<IConcertBrowserItem> ClientItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FClientBrowserItem>>, OwnerTable )
		.Padding(2.f)
		.Style(FConcertServerStyle::Get(), "Concert.Clients.TileTableRow")
		.Content()
		[
			SNew(SConcertBrowserItem, ClientItem.ToSharedRef())
			 .HighlightText(HighlightText)
		];
}

TSharedPtr<SWidget> UE::MultiUserServer::SConcertNetworkBrowser::OnGetContextMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenLog", "Open log"),
		LOCTEXT("OpenLogTooltip", "Opens a new tab in which you can filter log events related to this client"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				TArray<TSharedPtr<IConcertBrowserItem>> Items = TileView->GetSelectedItems();
				for (const TSharedPtr<IConcertBrowserItem>& Item : Items)
				{
					OnListMouseButtonDoubleClick(Item);
				}
			}),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked()),
		NAME_None,
		EUserInterfaceActionType::Button
		);
	
	MenuBuilder.AddSeparator();
	AddDisplayModeEntry(MenuBuilder, EConcertBrowserItemDisplayMode::NetworkGraph,
		LOCTEXT("DisplayMode.NetworkGraph.Title", "Network graph"),
		LOCTEXT("DisplayMode.NetworkGraph.Tooltip", "Show the up and down stream network traffic on a graph")
		);
	AddDisplayModeEntry(MenuBuilder, EConcertBrowserItemDisplayMode::OutboundSegementTable,
		LOCTEXT("DisplayMode.OutboundSegementTable.Title", "Outbound segment table"),
		LOCTEXT("DisplayMode.OutboundSegementTable.Tooltip", "A table displaying the messaging protocol's outbound segments' MessageId, Sent, Acked and Size data in realtime.")
		);
	AddDisplayModeEntry(MenuBuilder, EConcertBrowserItemDisplayMode::InboundSegmentTable,
		LOCTEXT("DisplayMode.InboundSegementTable.Title", "Inbound Segment table"),
		LOCTEXT("DisplayMode.InboundSegementTable.Tooltip", "A table displaying the messaging protocol's inbound segments' MessageId, Received and Size data in realtime.")
		);
	return MenuBuilder.MakeWidget();
}

void UE::MultiUserServer::SConcertNetworkBrowser::AddDisplayModeEntry(FMenuBuilder& MenuBuilder, EConcertBrowserItemDisplayMode DisplayMode, FText Title, FText Tooltip) const
{
	MenuBuilder.AddMenuEntry(
		Title,
		Tooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, DisplayMode]()
			{
				TArray<TSharedPtr<IConcertBrowserItem>> Items = TileView->GetSelectedItems();
				for (const TSharedPtr<IConcertBrowserItem>& Item : Items)
				{
					Item->SetDisplayMode(DisplayMode);
				}
			}),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this, DisplayMode]()
			{
				return Algo::AllOf(TileView->GetSelectedItems(), [this, DisplayMode](const TSharedPtr<IConcertBrowserItem>& Item)
				{
					return Item->GetDisplayMode() == DisplayMode;
				});
			})),
		NAME_None,
		EUserInterfaceActionType::Check
	);
}

void UE::MultiUserServer::SConcertNetworkBrowser::OnListMouseButtonDoubleClick(TSharedPtr<IConcertBrowserItem> Item)
{
	const int32 Index = DisplayedClientItems.IndexOfByPredicate([&Item](const TSharedPtr<FClientBrowserItem>& ClientItem)
	{
		return Item == ClientItem;
	});
	if (Index != INDEX_NONE)
	{
		OnClientDoubleClicked.ExecuteIfBound(DisplayedClientItems[Index]->GetMessageNodeId());
	}
	else
	{
		OnServerDoubleClicked.ExecuteIfBound();
	}
}

void UE::MultiUserServer::SConcertNetworkBrowser::SConcertNetworkBrowser::AllowAllSessions()
{
	AllowedSessions = BrowserModel->GetSessions();
	UpdateTileViewFromAllowedSessions();
}

void UE::MultiUserServer::SConcertNetworkBrowser::DisallowAllSessions()
{
	AllowedSessions.Reset();
	UpdateTileViewFromAllowedSessions();
}

void UE::MultiUserServer::SConcertNetworkBrowser::UpdateTileViewFromAllowedSessions()
{
	bShowAllSessions = AllowedSessions.Num() == BrowserModel->GetSessions().Num();
	DisplayedClientItems.Empty();
	DisplayedItems.Empty();
	
	if (PassesFilter(ServerItem))
	{
		DisplayedItems.Add(ServerItem);
	}
	
	for (const TSharedPtr<FClientBrowserItem>& Item : BrowserModel->GetItems())
	{
		if (PassesFilter(Item))
		{
			DisplayedClientItems.Add(Item);
			DisplayedItems.Add(Item);
		}
	}
	
	TileView->RequestListRefresh();
}

bool UE::MultiUserServer::SConcertNetworkBrowser::PassesFilter(const TSharedPtr<FServerBrowserItem>& Client) const
{
	return SessionFilter->PassesFilter(Client);
}

bool UE::MultiUserServer::SConcertNetworkBrowser::PassesFilter(const TSharedPtr<FClientBrowserItem>& Client) const
{
	const bool bIsSessionAllowed = Client->GetCurrentSession().IsSet() && AllowedSessions.Contains(*Client->GetCurrentSession());
	const bool bAllowBaseOnSession = bIsSessionAllowed || (bShowSessionlessClients && !Client->GetCurrentSession().IsSet());
	return bAllowBaseOnSession && SessionFilter->PassesFilter(Client);
}

void UE::MultiUserServer::SConcertNetworkBrowser::GenerateSearchTerms(const TSharedPtr<IConcertBrowserItem>& Client, TArray<FString>& SearchTerms) const
{
	Client->AppendSearchTerms(SearchTerms);
}


#undef LOCTEXT_NAMESPACE
