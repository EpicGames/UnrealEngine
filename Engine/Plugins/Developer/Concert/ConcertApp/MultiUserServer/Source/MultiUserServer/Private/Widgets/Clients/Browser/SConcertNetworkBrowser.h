// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Models/IClientBrowserModel.h"
#include "Widgets/SCompoundWidget.h"

class IConcertServer;
namespace UE::MultiUserServer { class IConcertBrowserItem; }
template <typename ItemType> class TTextFilter;

class FMenuBuilder;
class STableViewBase;
class ITableRow;
template<typename T> class STileView;

enum class EConcertClientStatus : uint8;
struct FConcertSessionClientInfo;

namespace UE::MultiUserServer
{
	class FClientBrowserItem;
	class FServerBrowserItem;
	class IClientNetworkStatisticsModel;
	class IClientBrowserModel;
	enum class EConcertBrowserItemDisplayMode : uint8;
	
	/**
	 * Presents a view of client thumbnails, which include important statistics.
	 * Clients can be right-clicked, opening a context menu, and double clicked, opening a new tab with a log view.
	 */
	class SConcertNetworkBrowser : public SCompoundWidget
	{
	public:

		DECLARE_DELEGATE_OneParam(FOnClientDoubleClicked, const FGuid& /*EndpointId*/);
		DECLARE_DELEGATE(FOnServerDoubleClicked);

		SLATE_BEGIN_ARGS(SConcertNetworkBrowser) {}
			/** Extension point to the right of the search bar */
			SLATE_NAMED_SLOT(FArguments, RightOfSearch)
			SLATE_EVENT(FOnServerDoubleClicked, OnServerDoubleClicked)
			SLATE_EVENT(FOnClientDoubleClicked, OnClientDoubleClicked)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<IClientBrowserModel> InBrowserModel, TSharedRef<IClientNetworkStatisticsModel> NetworkStatisticsModel, TSharedRef<IConcertServer> Server);

		/** Shows only the clients connected to the given session ID */
		void ShowOnlyClientsFromSession(const FGuid& SessionId);
		
	private:

		using FSessionId = FGuid;
		using FMessagingNodeId = FGuid;
		using FBrowserItemTextFilter = TTextFilter<const TSharedPtr<IConcertBrowserItem>&>;

		/** Retrieves clients and live sessions */
		TSharedPtr<IClientBrowserModel> BrowserModel;

		/** Contains the guid of every session we're allowed to display */
		TSet<FSessionId> AllowedSessions;
		/** Should all sessions be shown? */
		bool bShowAllSessions = true;
		/** Should admin endpoints be shown? */
		bool bShowSessionlessClients = true;

		TSharedPtr<FServerBrowserItem> ServerItem;
		/** Filtered version of IClientBrowserModel::GetItems. Added to DisplayItems. */
		TArray<TSharedPtr<FClientBrowserItem>> DisplayedClientItems;
		/** Source array for TileView - fitlered version of IClientBrowserModel::GetItems */
		TArray<TSharedPtr<IConcertBrowserItem>> DisplayedItems;
		/** Visualizes all the items */
		TSharedPtr<STileView<TSharedPtr<IConcertBrowserItem>>> TileView;

		// Filtering
		TSharedPtr<FText> HighlightText;
		TSharedPtr<FBrowserItemTextFilter> SessionFilter;
		
		FOnServerDoubleClicked OnServerDoubleClicked;
		FOnClientDoubleClicked OnClientDoubleClicked;

		TSharedRef<SWidget> CreateSearchArea(const FArguments& InArgs);
		TSharedRef<SWidget> CreateOptionsButton();
		TSharedRef<SWidget> CreateTileView();

		TSharedRef<SWidget> CreateOptionsButtonMenu();
		void ToggleShouldKeepDisconnected();

		// Model events
		void OnSessionCreated(const FGuid& SessionId);
		void OnSessionDestroyed(const FGuid& SessionId);
		void OnClientListChanged(TSharedPtr<FClientBrowserItem> Item, IClientBrowserModel::EClientUpdateType UpdateType);

		// Combo button
		TSharedRef<SWidget> CreateSessionFilterOptionMenu();

		FText GetErrorMessageText() const;
		
		// TileView events
		TSharedRef<ITableRow> MakeTileViewWidget(TSharedPtr<IConcertBrowserItem> ClientItem, const TSharedRef<STableViewBase>& OwnerTable);
		TSharedPtr<SWidget> OnGetContextMenuContent();
		void AddDisplayModeEntry(FMenuBuilder& MenuBuilder, EConcertBrowserItemDisplayMode DisplayMode, FText Title, FText Tooltip) const;
		void OnListMouseButtonDoubleClick(TSharedPtr<IConcertBrowserItem> ClientItem);

		// Filtering
		void AllowAllSessions();
		void DisallowAllSessions();
		void UpdateTileViewFromAllowedSessions();
		bool PassesFilter(const TSharedPtr<FServerBrowserItem>& Client) const;
		bool PassesFilter(const TSharedPtr<FClientBrowserItem>& Client) const;
		void GenerateSearchTerms(const TSharedPtr<IConcertBrowserItem>& Client, TArray<FString>& SearchTerms) const;
	};
}


