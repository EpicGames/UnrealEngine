// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClientInboundTransferStatTable.h"

#include "Framework/Views/TableViewMetadata.h"
#include "INetworkMessagingExtension.h"
#include "Widgets/Clients/Browser/Models/ITransferStatisticsModel.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SClientInboundTransferStatTable"

namespace UE::MultiUserServer
{
	class SClientInboundTransferStatTableRow : public SMultiColumnTableRow<TSharedRef<FOutboundTransferStatistics>>
	{
		using Super = SMultiColumnTableRow<TSharedRef<FOutboundTransferStatistics>>;
	public:
		
		inline static const FName HeaderIdName_MessageId			= TEXT("MessageId");
		inline static const FName HeaderIdName_ReceivedSegments	= TEXT("Received");
		inline static const FName HeaderIdName_TotalSize			= TEXT("Size");
		
		SLATE_BEGIN_ARGS(SClientInboundTransferStatTableRow) { }
			SLATE_ARGUMENT(TSharedPtr<FInboundTransferStatistics>, Stats)
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView)
		{
			Stats = InArgs._Stats;
			Super::Construct(Super::FArguments(), InOwerTableView);
		}
		
	private:

		TSharedPtr<FInboundTransferStatistics> Stats;

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			using namespace Private;
			if (HeaderIdName_MessageId == ColumnName)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(Stats->MessageId))
					];
			}
			if (HeaderIdName_ReceivedSegments == ColumnName)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text_Lambda([this](){ return FText::AsNumber(Stats->BytesReceived); })
					];
			}
			if (HeaderIdName_TotalSize == ColumnName)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(Stats->BytesToReceive))
					];
			}
			return SNullWidget::NullWidget;
		}
	};
	
	void SClientInboundTransferStatTable::Construct(const FArguments& InArgs, TSharedRef<ITransferStatisticsModel> InStatsModel)
	{
		StatsModel = InStatsModel;

		StatsModel->OnOutboundTransferGroupsUpdated().AddSP(this, &SClientInboundTransferStatTable::OnTransferStatisticsUpdated);

		ChildSlot
		[
			SAssignNew(SegmenterListView, SListView<TSharedPtr<FInboundTransferStatistics>>)
			.ListItemsSource(&InStatsModel->GetInboundTransferStatsGroupedById())
			.OnGenerateRow(this, &SClientInboundTransferStatTable::OnGenerateActivityRowWidget)
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(SClientInboundTransferStatTableRow::HeaderIdName_MessageId)
				.FillWidth(2.f)
				.DefaultLabel(LOCTEXT("HeaderName_MessageId", "Id"))
				+ SHeaderRow::Column(SClientInboundTransferStatTableRow::HeaderIdName_ReceivedSegments)
				.FillWidth(2.f)
				.DefaultLabel(LOCTEXT("HeaderName_AckSegments", "Received"))
				+ SHeaderRow::Column(SClientInboundTransferStatTableRow::HeaderIdName_TotalSize)
				.FillWidth(2.f)
				.DefaultLabel(LOCTEXT("HeaderName_TotalSize", "Size"))
			)
		];
	}

	TSharedRef<ITableRow> SClientInboundTransferStatTable::OnGenerateActivityRowWidget(TSharedPtr<FInboundTransferStatistics> InStats, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return SNew(SClientInboundTransferStatTableRow, OwnerTable)
			.Stats(InStats);;
	}

	void SClientInboundTransferStatTable::OnTransferStatisticsUpdated() const
	{
		SegmenterListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
