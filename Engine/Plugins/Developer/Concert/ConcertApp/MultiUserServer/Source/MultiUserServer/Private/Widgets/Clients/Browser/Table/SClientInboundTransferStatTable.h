// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

struct FInboundTransferStatistics;

class ITableRow;
template<typename T> class SListView;
class STableViewBase;
struct FOutboundTransferStatistics;

namespace UE::MultiUserServer
{
	class ITransferStatisticsModel;

	/**
	 * Displays FOutboundTransferStatistics as they are updated by INetworkMessagingExtension.
	 */
	class SClientInboundTransferStatTable : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SClientInboundTransferStatTable)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<ITransferStatisticsModel> InStatsModel);

	private:

		/** Tells us when the transfer stats have changed */
		TSharedPtr<ITransferStatisticsModel> StatsModel;
		/** Displays the transfer stats */
		TSharedPtr<SListView<TSharedPtr<FInboundTransferStatistics>>> SegmenterListView;

		TSharedRef<ITableRow> OnGenerateActivityRowWidget(TSharedPtr<FInboundTransferStatistics> InStats, const TSharedRef<STableViewBase>& OwnerTable) const;
		void OnTransferStatisticsUpdated() const;
	};
}


