// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClientOutboundTransferStatTable.h"

#include "Framework/Views/TableViewMetadata.h"
#include "INetworkMessagingExtension.h"
#include "Widgets/Clients/Browser/Models/ITransferStatisticsModel.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SClientTransferStatTable"

namespace UE::MultiUserServer
{
	namespace Private
	{
		const FName HeaderIdName_MessageId    = TEXT("MessageId");
		const FName HeaderIdName_SentSegments = TEXT("Sent");
		const FName HeaderIdName_AckSegments  = TEXT("SegmentsAck");
		const FName HeaderIdName_TotalSize    = TEXT("Size");
		const FName HeaderIdName_DataRate     = TEXT("DataRate");
	}

	class SClientOutboundTransferStatTableRow : public SMultiColumnTableRow<TSharedRef<FOutboundTransferStatistics>>
	{
		using Super = SMultiColumnTableRow<TSharedRef<FOutboundTransferStatistics>>;
	public:
		
		SLATE_BEGIN_ARGS(SClientOutboundTransferStatTableRow) { }
			SLATE_ARGUMENT(TSharedPtr<FOutboundTransferStatistics>, Stats)
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView)
		{
			Stats = InArgs._Stats;
			Super::Construct(Super::FArguments(), InOwerTableView);
		}
		
	private:

		TSharedPtr<FOutboundTransferStatistics> Stats;

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			using namespace Private;
			if (HeaderIdName_MessageId == ColumnName)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(Stats->MessageId))
					];
			}
			if (HeaderIdName_SentSegments == ColumnName)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text_Lambda([this](){ return FText::AsNumber(Stats->BytesSent); })
					];
			}
			if (HeaderIdName_AckSegments == ColumnName)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text_Lambda([this](){ return FText::AsNumber(Stats->BytesAcknowledged); })
					];
			}
			if (HeaderIdName_TotalSize == ColumnName)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(Stats->BytesToSend))
					];
			}
			return SNullWidget::NullWidget;
		}
	};
	
	void SClientOutboundTransferStatTable::Construct(const FArguments& InArgs, TSharedRef<ITransferStatisticsModel> InStatsModel)
	{
		StatsModel = InStatsModel;

		StatsModel->OnOutboundTransferGroupsUpdated().AddSP(this, &SClientOutboundTransferStatTable::OnTransferStatisticsUpdated);

		ChildSlot
		[
			SAssignNew(SegmenterListView, SListView<TSharedPtr<FOutboundTransferStatistics>>)
			.ListItemsSource(&InStatsModel->GetOutboundTransferStatsGroupedById())
			.OnGenerateRow(this, &SClientOutboundTransferStatTable::OnGenerateActivityRowWidget)
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(Private::HeaderIdName_MessageId)
				.FillWidth(2.f)
				.DefaultLabel(LOCTEXT("HeaderName_MessageId", "Id"))
				+ SHeaderRow::Column(Private::HeaderIdName_SentSegments)
				.FillWidth(2.f)
				.DefaultLabel(LOCTEXT("HeaderName_SentSegments", "Sent"))
				+ SHeaderRow::Column(Private::HeaderIdName_AckSegments)
				.FillWidth(2.f)
				.DefaultLabel(LOCTEXT("HeaderName_AckSegments", "Ack"))
				+ SHeaderRow::Column(Private::HeaderIdName_TotalSize)
				.FillWidth(2.f)
				.DefaultLabel(LOCTEXT("HeaderName_TotalSize", "Size"))
			)
		];
	}

	TSharedRef<ITableRow> SClientOutboundTransferStatTable::OnGenerateActivityRowWidget(TSharedPtr<FOutboundTransferStatistics> InStats, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return SNew(SClientOutboundTransferStatTableRow, OwnerTable)
			.Stats(InStats);;
	}

	void SClientOutboundTransferStatTable::OnTransferStatisticsUpdated() const
	{
		SegmenterListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
