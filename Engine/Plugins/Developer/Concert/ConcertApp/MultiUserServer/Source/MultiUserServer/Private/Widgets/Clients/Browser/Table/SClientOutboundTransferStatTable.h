// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
template<typename T> class SListView;
class STableViewBase;
struct FOutboundTransferStatistics;

namespace UE::MultiUserServer
{
	class ITransferStatisticsModel;

	/**
	 * Displays FOutboundTransferStatistics as they are updated by INetworkMessagingExtension.
	 */
	class SClientOutboundTransferStatTable : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SClientOutboundTransferStatTable)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<ITransferStatisticsModel> InStatsModel);

	private:

		/** Tells us when the transfer stats have changed */
		TSharedPtr<ITransferStatisticsModel> StatsModel;
		/** Displays the transfer stats */
		TSharedPtr<SListView<TSharedPtr<FOutboundTransferStatistics>>> SegmenterListView;

		TSharedRef<ITableRow> OnGenerateActivityRowWidget(TSharedPtr<FOutboundTransferStatistics> InStats, const TSharedRef<STableViewBase>& OwnerTable) const;
		void OnTransferStatisticsUpdated() const;
	};
}


