// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientsTabController.h"

#include "ConcertServerStyle.h"
#include "IConcertModule.h"
#include "IConcertSyncServer.h"
#include "SConcertClientsTabView.h"
#include "Logging/Source/GlobalLogSource.h"
#include "Logging/Util/LogAckTracker.h"
#include "Widgets/SConcertTabViewBase.h"
#include "Window/ConcertServerTabs.h"
#include "Window/ConcertServerWindowController.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"

namespace UE::MultiUserServer
{
	static constexpr uint64 GlobalLogCapacity = 5000000;
	
	void FConcertClientsTabController::Init(const FConcertComponentInitParams& Params)
	{
		LogBuffer = MakeShared<FGlobalLogSource>(UE::MultiUserServer::GlobalLogCapacity);
		AckTracker = MakeShared<FLogAckTracker>(LogBuffer.ToSharedRef(), Params.Server->GetConcertServer());
	
		FGlobalTabmanager::Get()->RegisterTabSpawner(
					ConcertServerTabs::GetClientsTabID(),
					FOnSpawnTab::CreateRaw(this, &FConcertClientsTabController::SpawnClientsTab, Params.WindowController->GetRootWindow(), Params.Server)
				)
				.SetDisplayName(LOCTEXT("ClientsTabTitle", "Clients"))
				.SetTooltipText(LOCTEXT("ClientsTooltipText", "View network statistics for connected clients."))
				.SetIcon(FSlateIcon(FConcertServerStyle::GetStyleSetName(), TEXT("Concert.Icon.Client"))
			);
		Params.MainStack->AddTab(ConcertServerTabs::GetClientsTabID(), ETabState::OpenedTab);
	}

	void FConcertClientsTabController::ShowConnectedClients(const FGuid& SessionId) const
	{
		FGlobalTabmanager::Get()->TryInvokeTab(ConcertServerTabs::GetClientsTabID());
		ClientsView->ShowConnectedClients(SessionId);
	}

	TSharedRef<SDockTab> FConcertClientsTabController::SpawnClientsTab(const FSpawnTabArgs& Args, TSharedPtr<SWindow> RootWindow, TSharedRef<IConcertSyncServer> Server)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label(LOCTEXT("ClientsTabTitle", "Clients"))
			.TabRole(MajorTab)
			.CanEverClose(false);
		DockTab->SetContent(
			SAssignNew(ClientsView, SConcertClientsTabView, ConcertServerTabs::GetClientsTabID(), Server, LogBuffer.ToSharedRef())
				.ConstructUnderMajorTab(DockTab)
				.ConstructUnderWindow(RootWindow)
			);
		return DockTab;
	}
}

#undef LOCTEXT_NAMESPACE
