// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Window/IConcertComponent.h"

class FSpawnTabArgs;

class FLogAckTracker;
class FGlobalLogSource;
class SWindow;

namespace UE::MultiUserServer
{
	class SConcertClientsTabView;
	
	class FConcertClientsTabController : public IConcertComponent, public TSharedFromThis<FConcertClientsTabController>
	{
	public:

		//~ Begin IConcertComponent Interface
		virtual void Init(const FConcertComponentInitParams& Params) override;
		//~ End IConcertComponent Interface

		/** Highlights this tab and sets the client filter such that all connected clients of the given session ID are shown.  */
		void ShowConnectedClients(const FGuid& SessionId) const;

	private:

		/** Buffers generated logs up to a limit (and overrides oldest logs when buffer is full) */
		TSharedPtr<FGlobalLogSource> LogBuffer;
		/** Marks logs as acked as the ACKs come in. */
		TSharedPtr<FLogAckTracker> AckTracker;
	
		/** Manages the sub-tabs */
		TSharedPtr<SConcertClientsTabView> ClientsView;
	
		TSharedRef<SDockTab> SpawnClientsTab(const FSpawnTabArgs& Args, TSharedPtr<SWindow> RootWindow, TSharedRef<IConcertSyncServer>);
	};
}
