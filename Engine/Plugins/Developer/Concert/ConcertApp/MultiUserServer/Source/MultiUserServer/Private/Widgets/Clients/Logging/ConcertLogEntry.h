// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertTransportEvents.h"
#include "ConcertLogEntry.generated.h"

using FConcertLogID = uint64;

UENUM()
enum class EConcertLogAckState : uint8
{
	/** This message does not require any acks */
	NotNeeded,
	/** This is an ack */
	Ack,
	/** Still waiting on the ack */
	InProgress,
	/** The ack was received */
	AckReceived,
	/** Never received any acks - set after a certain timeout. */
	AckFailure
};

/** Additional properties we display in the UI but that are not in FConcertLog live here. */
USTRUCT()
struct FConcertLogMetadata
{
	GENERATED_BODY()

	/** The ack state of this log */
	UPROPERTY()
	EConcertLogAckState AckState = EConcertLogAckState::NotNeeded;

	/**
	 * Message ID of this log's ACK. If set, implies this log is not an ACK.
	 * Valid when AckState == EConcertLogAckState::AckReceived.
	 */
	TOptional<FGuid> AckingMessageId;

	/**
	 * The Message IDs of the logs we've ACKed. If set, implies this log is an ACK.
	 * Valid when AckState == EConcertLogAckState::Ack.
	 */
	TOptional<TSet<FGuid>> AckedMessageId;
};

struct FConcertLogEntry
{
	/** Unique log ID. Log IDs grow sequentially. */
	FConcertLogID LogId;

	/** The log we're describing */
	FConcertLog Log;

	/** Additional data to display in the UI */
	FConcertLogMetadata LogMetaData;
};

using FConcertLogEntryFilterFunc = TFunctionRef<bool(const FConcertLogEntry&)>;
