// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendLogFilter_BaseSetSelection.h"
#include "Widgets/Clients/Logging/ConcertLogEntry.h"

namespace UE::MultiUserServer
{
	/** Allows only messages that have a selected ACK state */
	class FConcertLogFilter_Ack : public TConcertLogFilter_BaseSetSelection<FConcertLogFilter_Ack, EConcertLogAckState>
	{
	public:

		static TSet<EConcertLogAckState> GetAllOptions()
		{
			return {
				EConcertLogAckState::Ack,
				EConcertLogAckState::AckFailure,
				EConcertLogAckState::AckReceived,
				EConcertLogAckState::InProgress,
				EConcertLogAckState::NotNeeded
			};
		}
		
		static FString GetOptionDisplayString(const EConcertLogAckState& Item)
		{
			switch(Item)
			{
				case EConcertLogAckState::Ack: return TEXT("Ack");
				case EConcertLogAckState::AckFailure: return TEXT("Ack Failure");
				case EConcertLogAckState::AckReceived: return TEXT("Ack Received");
				case EConcertLogAckState::InProgress: return TEXT("In Progress");
				case EConcertLogAckState::NotNeeded: return TEXT("Not needed");
				default: checkNoEntry(); return {}; 
			}
		}
		
		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(const FConcertLogEntry& InItem) const override
		{
			return IsItemAllowed(InItem.LogMetaData.AckState);
		}
		//~ End FConcertLogFilter Interface
	};

	class FConcertFrontendLogFilter_Ack : public TConcertFrontendLogFilter_BaseSetSelection<FConcertLogFilter_Ack>
	{
	public:

		FConcertFrontendLogFilter_Ack(TSharedRef<FFilterCategory> FilterCategory)
			: TConcertFrontendLogFilter_BaseSetSelection<FConcertLogFilter_Ack>(MoveTemp(FilterCategory), NSLOCTEXT("UnrealMultiUserUI.Filter.Ack", "Name", "ACKs"))
		{}
		
		virtual FString GetName() const override { return TEXT("Acknowledgement"); }
		virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealMultiUserUI.FConcertFrontendLogFilter_Ack", "DisplayLabel", "Acknowledgement"); }
	};
}


