// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Util/Filter/ConcertFrontendFilter.h"

class FFilterCategory;

struct FConcertLogEntry;

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.BaseSetSelection"

namespace UE::MultiUserServer
{
	/**
	 * Helper implementation for filters that allow filter based on a finte set of items.
	 * Subclasses must implement:
	 * - static TSet<TSetType> GetAllOptions();
	 * - static FString GetOptionDisplayString(const TSetType&);
	 */
	template<typename TRealFilterImpl, typename TSetType>
	class TConcertLogFilter_BaseSetSelection : public IFilter<const FConcertLogEntry&>
	{
		using Super = IFilter<TRealFilterImpl>;
	public:

		using TItemType = TSetType;

		TConcertLogFilter_BaseSetSelection()
			: AllowedItems(TRealFilterImpl::GetAllOptions())
		{}

		void AllowAll()
		{
			TSet<TSetType> Allowed = TRealFilterImpl::GetAllOptions();
			if (Allowed.Num() != AllowedItems.Num())
			{
				AllowedItems = MoveTemp(Allowed);
				OnChanged().Broadcast();
			}
		}
		void DisallowAll()
		{
			if (AllowedItems.Num() > 0)
			{
				AllowedItems.Reset();
				OnChanged().Broadcast();
			}
		}
		void ToggleAll(const TSet<TSetType>& ToToggle)
		{
			for (const TSetType Item : ToToggle)
			{
				if (IsItemAllowed(Item))
				{
					DisallowItem(Item);
				}
				else
				{
					AllowItem(Item);
				}
			}

			if (ToToggle.Num() > 0)
			{
				OnChanged().Broadcast();
			}
		}
	
		void AllowItem(const TSetType& MessageTypeName)
		{
			if (!AllowedItems.Contains(MessageTypeName))
			{
				AllowedItems.Add(MessageTypeName);
				OnChanged().Broadcast();
			}
		}
		void DisallowItem(const TSetType& MessageTypeName)
		{
			if (AllowedItems.Contains(MessageTypeName))
			{
				AllowedItems.Remove(MessageTypeName);
				OnChanged().Broadcast();
			}
		}
	
		bool IsItemAllowed(const TSetType& MessageTypeName) const
		{
			return AllowedItems.Contains(MessageTypeName);
		}
		bool AreAllAllowed() const
		{
			return AllowedItems.Num() == TRealFilterImpl::GetAllOptions().Num();
		}
		uint8 GetNumSelected() const { return AllowedItems.Num(); }

		virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

	private:
	
		TSet<TSetType> AllowedItems;
		FChangedEvent ChangedEvent;
	};

	template<typename TRealFilterImpl>
	class TConcertFrontendLogFilter_BaseSetSelection : public TConcertFrontendFilterAggregate<TRealFilterImpl, const FConcertLogEntry&>
	{
		using TItemType = typename TRealFilterImpl::TItemType;
		using Super = TConcertFrontendFilterAggregate<TRealFilterImpl, const FConcertLogEntry&>;
	public:

		TConcertFrontendLogFilter_BaseSetSelection(TSharedRef<FFilterCategory> FilterCategory, FText FilterName)
			: Super(MoveTemp(FilterCategory))
		{}

		virtual void ExposeEditWidgets(FMenuBuilder& MenuBuilder) override
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("BaseSetSelection.SelectAll.", "Select all"),
				LOCTEXT("BaseSetSelection.SelectAll.Tooltip", "Allows all items"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this](){ this->Implementation.AllowAll(); }),
					FCanExecuteAction::CreateLambda([] { return true; }),
					FIsActionChecked()),
				NAME_None,
				EUserInterfaceActionType::Button
				);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("BaseSetSelection.DeselectAll.", "Deselect all"),
				LOCTEXT("BaseSetSelection.DeelectAll.Tooltip", "Disallows all items"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this](){ this->Implementation.DisallowAll(); }),
					FCanExecuteAction::CreateLambda([] { return true; }),
					FIsActionChecked()),
				NAME_None,
				EUserInterfaceActionType::Button
				);
	
			for (const TItemType& Item : TRealFilterImpl::GetAllOptions())
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(TRealFilterImpl::GetOptionDisplayString(Item)),
					FText::GetEmpty(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, Item]()
						{
							if (this->Implementation.IsItemAllowed(Item))
							{
								this->Implementation.DisallowItem(Item);
							}
							else
							{
								this->Implementation.AllowItem(Item);
							}
						}),
						FCanExecuteAction::CreateLambda([] { return true; }),
						FIsActionChecked::CreateLambda([this, Item]() { return this->Implementation.IsItemAllowed(Item); })),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
		}
	};

}

#undef LOCTEXT_NAMESPACE
