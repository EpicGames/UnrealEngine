// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertFrontendLogFilter_Client.h"

#include "Widgets/Clients/Logging/ConcertLogEntry.h"
#include "Widgets/Clients/Util/EndpointToUserNameCache.h"

namespace UE::MultiUserServer
{
	bool FConcertLogFilter_Client::PassesFilter(const FConcertLogEntry& InItem) const
	{
		return AllowedClientMessagingNodeId == EndpointCache->TranslateEndpointIdToNodeId(InItem.Log.OriginEndpointId)
			|| AllowedClientMessagingNodeId == EndpointCache->TranslateEndpointIdToNodeId(InItem.Log.DestinationEndpointId);
	}
}
