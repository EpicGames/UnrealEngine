// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Misc/IFilter.h"

struct FConcertLogEntry;

class FEndpointToUserNameCache;

namespace UE::MultiUserServer
{
	/** Only allows messages from the given clients */
	class FConcertLogFilter_Client : public IFilter<const FConcertLogEntry&>
	{
	public:

		explicit FConcertLogFilter_Client(const FGuid& SingleAllowedId, TSharedRef<FEndpointToUserNameCache> EndpointCache)
			: AllowedClientMessagingNodeId(SingleAllowedId)
			, EndpointCache(MoveTemp(EndpointCache))
		{}

		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(const FConcertLogEntry& InItem) const override;
		virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
		//~ End FConcertLogFilter Interface

	private:

		/** Messages to and from the following client endpoint ID are allowed */
		FGuid AllowedClientMessagingNodeId;

		/** Translates Concert endpoint Ids to messaging node Ids */
		TSharedRef<FEndpointToUserNameCache> EndpointCache;

		FChangedEvent ChangedEvent;
	};
}
