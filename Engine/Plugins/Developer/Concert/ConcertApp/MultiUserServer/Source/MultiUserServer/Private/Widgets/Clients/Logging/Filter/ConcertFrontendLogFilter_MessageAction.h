// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendLogFilter_BaseSetSelection.h"
#include "Widgets/Clients/Logging/ConcertLogEntry.h"
#include "Widgets/Clients/Logging/Util/MessageActionUtils.h"

namespace UE::MultiUserServer
{
	/** Allows only the selected messages types */
	class FConcertLogFilter_MessageAction : public TConcertLogFilter_BaseSetSelection<FConcertLogFilter_MessageAction, FName>
	{
	public:

		static TSet<FName> GetAllOptions()
		{
			return MessageActionUtils::GetAllMessageActionNames();
		}
		
		static FString GetOptionDisplayString(const FName& Item)
		{
			return MessageActionUtils::GetActionDisplayString(Item);
		}

		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(const FConcertLogEntry& InItem) const override
		{
			return IsItemAllowed(MessageActionUtils::ConvertActionToName(InItem.Log.MessageAction));
		}
		//~ End FConcertLogFilter Interface
	};

	class FConcertFrontendLogFilter_MessageAction : public TConcertFrontendLogFilter_BaseSetSelection<FConcertLogFilter_MessageAction>
	{
	public:

		FConcertFrontendLogFilter_MessageAction(TSharedRef<FFilterCategory> FilterCategory)
			: TConcertFrontendLogFilter_BaseSetSelection<FConcertLogFilter_MessageAction>(MoveTemp(FilterCategory), NSLOCTEXT("UnrealMultiUserUI.Filter.MessageAction", "Name", "Actions"))
		{}
		
		virtual FString GetName() const override { return TEXT("Message Action"); }
		virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealMultiUserUI.FConcertFrontendLogFilter_MessageAction", "DisplayLabel", "Action"); }
	};
}

