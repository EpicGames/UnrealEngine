// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendLogFilter_BaseSetSelection.h"
#include "Widgets/Clients/Logging/ConcertLogEntry.h"
#include "Widgets/Clients/Logging/Util/MessageTypeUtils.h"

namespace UE::MultiUserServer
{
	/** Allows only the selected messages types */
	class FConcertLogFilter_MessageType : public TConcertLogFilter_BaseSetSelection<FConcertLogFilter_MessageType, FName>
	{
	public:

		static TSet<FName> GetAllOptions()
		{
			return MessageTypeUtils::GetAllMessageTypeNames();
		}
		
		static FString GetOptionDisplayString(const FName& Item)
		{
			return MessageTypeUtils::SanitizeMessageTypeName(Item);
		}
		
		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(const FConcertLogEntry& InItem) const override
		{
			return IsItemAllowed(InItem.Log.MessageTypeName);
		}
		//~ End FConcertLogFilter Interface
	
	private:
	
		TSet<FName> AllowedMessageTypeNames;
	};

	class FConcertFrontendLogFilter_MessageType : public TConcertFrontendLogFilter_BaseSetSelection<FConcertLogFilter_MessageType>
	{
	public:

		FConcertFrontendLogFilter_MessageType(TSharedRef<FFilterCategory> FilterCategory)
			: TConcertFrontendLogFilter_BaseSetSelection<FConcertLogFilter_MessageType>(MoveTemp(FilterCategory), NSLOCTEXT("UnrealMultiUserUI.Filter.MessageType", "Name", "Message Type"))
		{}
		
		virtual FString GetName() const override { return TEXT("Message Type"); }
		virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealMultiUserUI.FConcertFrontendLogFilter_MessageType", "DisplayLabel", "Type"); }
	};
}

