// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertFrontendLogFilter_Size.h"

#include "Widgets/Clients/Logging/ConcertLogEntry.h"
#include "Widgets/Util/Filter/ConcertFilterUtils.h"

#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FConcertLogFilter_Size"

namespace UE::MultiUserServer
{
	bool FConcertLogFilter_Size::PassesFilter(const FConcertLogEntry& InItem) const
	{
		check(InItem.Log.CustomPayloadUncompressedByteSize >= 0);
		// Note: This only filters activities events - they all use custom events.
		// The filter's default value is to show everything 0 <= x so it will show sync events as well.
		const uint32 ComparisionSizeInBytes = FUnitConversion::Convert(SizeInBytes, DataUnit, EUnit::Bytes);
		switch (FilterMode)
		{
		case ESizeFilterMode::LessThanOrEqual:
			return ComparisionSizeInBytes >= static_cast<uint32>(InItem.Log.CustomPayloadUncompressedByteSize);
		case ESizeFilterMode::BiggerThanOrEqual:
			return ComparisionSizeInBytes <= static_cast<uint32>(InItem.Log.CustomPayloadUncompressedByteSize);
		default:
			checkNoEntry();
			return false;
		}
	}

	void FConcertLogFilter_Size::SetOperator(ESizeFilterMode Operator)
	{
		if (FilterMode != Operator)
		{
			FilterMode = Operator;
			OnChanged().Broadcast();
		}
	}

	void FConcertLogFilter_Size::SetSizeInBytes(uint32 NewSizeInBytes)
	{
		if (NewSizeInBytes != SizeInBytes)
		{
			SizeInBytes = NewSizeInBytes;
			OnChanged().Broadcast();
		}
	}

	void FConcertLogFilter_Size::SetDataUnit(EUnit NewUnit)
	{
		if (DataUnit != NewUnit && ensure(GetAllowedUnits().Contains(NewUnit)))
		{
			DataUnit = NewUnit;
			OnChanged().Broadcast();
		}
	}

	FConcertFrontendLogFilter_Size::FConcertFrontendLogFilter_Size(TSharedRef<FFilterCategory> FilterCategory)
		: Super(MoveTemp(FilterCategory))
	{}

	void FConcertFrontendLogFilter_Size::ExposeEditWidgets(FMenuBuilder& MenuBuilder)
	{
		using namespace ConcertFilterUtils;

		constexpr float MenuWidth = 100.f;
		TRadialMenuBuilder<ESizeFilterMode>::AddRadialSubMenu(
			MenuBuilder,
			LOCTEXT("Operator", "Operator"),
			TAttribute<ESizeFilterMode>::CreateLambda([this](){ return Implementation.GetOperator(); }),
			TArray<ESizeFilterMode>{ ESizeFilterMode::BiggerThanOrEqual, ESizeFilterMode::LessThanOrEqual },
			TRadialMenuBuilder<ESizeFilterMode>::FItemToText::CreateLambda([this](const ESizeFilterMode& Operator)
			{
				switch (Operator)
				{
				case ESizeFilterMode::BiggerThanOrEqual: return LOCTEXT("BiggerThanOrEqual.Text", ">= (Greater)") ;
				case ESizeFilterMode::LessThanOrEqual: return LOCTEXT("LessThanOrEqual.Text", "<= (Less)") ;
				default: return FText::GetEmpty();
				}
			}),
			TRadialMenuBuilder<ESizeFilterMode>::FOnSelectItem::CreateLambda([this](const ESizeFilterMode& Operator)
			{
				Implementation.SetOperator(Operator);
			}), MenuWidth);

		const TSharedRef<SWidget> SizeWidget = SNew(SNumericEntryBox<uint32>)
			.AllowSpin(true)
			.MinDesiredValueWidth(30)
			.MaxSliderValue(1000)
			.OnValueChanged_Lambda([this](uint32 NewValue){ Implementation.SetSizeInBytes(NewValue); })
			.Value_Lambda([this](){ return Implementation.GetSizeInBytes(); });
		MenuBuilder.AddWidget(SetMenuWidgetWidth(SizeWidget), LOCTEXT("Size", "Size"), true);

		TRadialMenuBuilder<EUnit>::AddRadialSubMenu(
			MenuBuilder,
			LOCTEXT("Unit", "Unit"),
			TAttribute<EUnit>::CreateLambda([this](){ return Implementation.GetDataUnit(); }),
			TArray<EUnit>{ EUnit::Bytes, EUnit::Kilobytes, EUnit::Megabytes },
			TRadialMenuBuilder<EUnit>::FItemToText::CreateLambda([this](const EUnit& Unit)
			{
				return FText::FromString(FUnitConversion::GetUnitDisplayString(Unit));
			}),
			TRadialMenuBuilder<EUnit>::FOnSelectItem::CreateLambda([this](const EUnit& Unit)
			{
				Implementation.SetDataUnit(Unit);
			}), MenuWidth);
	}
}

#undef LOCTEXT_NAMESPACE
