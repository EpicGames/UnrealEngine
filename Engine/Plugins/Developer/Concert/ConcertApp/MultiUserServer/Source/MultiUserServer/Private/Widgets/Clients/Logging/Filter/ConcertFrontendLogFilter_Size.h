// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/Util/Filter/ConcertFrontendFilter.h"

#include "Math/UnitConversion.h"

struct FConcertLogEntry;

namespace UE::MultiUserServer
{
	enum class ESizeFilterMode
	{
		/** Allow logs bigger than or equal to the specified size */
		BiggerThanOrEqual,
		/** Allow logs smaller than or equal to the specified size */
		LessThanOrEqual
	};

	/** Filters based on the log's size */
	class FConcertLogFilter_Size : public IFilter<const FConcertLogEntry&>
	{
	public:
	
		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(const FConcertLogEntry& InItem) const override;
		virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
		//~ End FConcertLogFilter Interface

		void SetOperator(ESizeFilterMode Operator);
		void SetSizeInBytes(uint32 NewSizeInBytes);
		void SetDataUnit(EUnit NewUnit);
	
		ESizeFilterMode GetOperator() const { return FilterMode; }
		uint32 GetSizeInBytes() const { return SizeInBytes; }
		EUnit GetDataUnit() const { return DataUnit; }
		TSet<EUnit> GetAllowedUnits() const { return { EUnit::Bytes, EUnit::Kilobytes, EUnit::Megabytes }; }
	
	private:

		ESizeFilterMode FilterMode = ESizeFilterMode::BiggerThanOrEqual;
		uint32 SizeInBytes = 0;
		EUnit DataUnit = EUnit::Bytes;

		FChangedEvent ChangedEvent;
	};

	class FConcertFrontendLogFilter_Size : public TConcertFrontendFilterAggregate<FConcertLogFilter_Size, const FConcertLogEntry&>
	{
		using Super = TConcertFrontendFilterAggregate<FConcertLogFilter_Size, const FConcertLogEntry&>;
	public:
	
		FConcertFrontendLogFilter_Size(TSharedRef<FFilterCategory> FilterCategory);

		virtual FString GetName() const override { return TEXT("Size"); }
		virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealMultiUserUI.FConcertFrontendLogFilter_Size", "DisplayLabel", "Size"); }

		virtual void ExposeEditWidgets(FMenuBuilder& MenuBuilder) override;
	};
}
