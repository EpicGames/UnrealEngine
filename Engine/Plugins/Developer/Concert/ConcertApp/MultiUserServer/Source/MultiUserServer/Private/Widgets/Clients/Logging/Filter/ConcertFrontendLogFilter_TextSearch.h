// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Clients/Logging/ConcertLogEntry.h"
#include "Widgets/Clients/Logging/Util/ConcertLogTokenizer.h"
#include "Widgets/Util/Filter/ConcertFrontendFilter.h"
#include "Widgets/Util/Filter/ConcertFrontendFilter_TextSearch.h"

class FConcertLogTokenizer;

namespace UE::MultiUserServer
{
	class FConcertLogFilter_TextSearch : public TConcertFilter_TextSearch<const FConcertLogEntry&>
	{
	public:

		FConcertLogFilter_TextSearch(TSharedRef<FConcertLogTokenizer> Tokenizer)
			: Tokenizer(MoveTemp(Tokenizer))
		{}
	
		virtual void GenerateSearchTerms(const FConcertLogEntry& InItem, TArray<FString>& OutTerms) const override
		{
			for (TFieldIterator<const FProperty> PropertyIt(FConcertLog::StaticStruct()); PropertyIt; ++PropertyIt)
			{
				OutTerms.Add(Tokenizer->Tokenize(InItem.Log, **PropertyIt));
			}
		}
	
	private:

		/** Helps in converting FConcertLog members into search terms */
		TSharedRef<FConcertLogTokenizer> Tokenizer;
	};

	/** Creates a search bar */
	class FConcertFrontendLogFilter_TextSearch : public TConcertFrontendFilter_TextSearch<FConcertLogFilter_TextSearch, const FConcertLogEntry&>
	{
		using Super = TConcertFrontendFilter_TextSearch<FConcertLogFilter_TextSearch, const FConcertLogEntry&>;
	public:

		FConcertFrontendLogFilter_TextSearch(TSharedPtr<FFilterCategory> InCategory, TSharedRef<FConcertLogTokenizer> Tokenizer)
			: Super(MoveTemp(InCategory), MoveTemp(Tokenizer))
		{}
	};
}


