// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertFrontendLogFilter_Time.h"

#include "Widgets/Clients/Logging/ConcertLogEntry.h"

#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Util/Filter/ConcertFilterUtils.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FConcertFrontendLogFilter_Time"

namespace UE::MultiUserServer
{
	FConcertLogFilter_Time::FConcertLogFilter_Time(ETimeFilter FilterMode)
		: FilterMode(FilterMode)
		// Make the filter allow everything by default
		, Time(MakeResetTime())
	{}

	void FConcertLogFilter_Time::ResetToInfiniteTime()
	{
		SetTime(MakeResetTime());
	}

	FDateTime FConcertLogFilter_Time::MakeResetTime() const
	{
		return FilterMode == ETimeFilter::AllowAfter ? FDateTime() : FDateTime::MaxValue();
	}

	bool FConcertLogFilter_Time::PassesFilter(const FConcertLogEntry& InItem) const
	{
		switch (FilterMode)
		{
		case ETimeFilter::AllowAfter: return InItem.Log.Timestamp >= Time;
		case ETimeFilter::AllowBefore: return InItem.Log.Timestamp <= Time;
		default:
			checkNoEntry();
			return true;
		}
	}

	void FConcertLogFilter_Time::SetFilterMode(ETimeFilter InFilterMode)
	{
		if (FilterMode != InFilterMode)
		{
			FilterMode = InFilterMode;
			OnChanged().Broadcast();
		}
	}

	void FConcertLogFilter_Time::SetTime(const FDateTime& InTime)
	{
		if (Time != InTime)
		{
			Time = InTime;
			OnChanged().Broadcast();
		}
	}

	FConcertFrontendLogFilter_Time::FConcertFrontendLogFilter_Time(TSharedRef<FFilterCategory> FilterCategory, ETimeFilter TimeFilter)
		: Super(MoveTemp(FilterCategory), TimeFilter)
		, TimeFilter(TimeFilter)
	{}

	void FConcertFrontendLogFilter_Time::ExposeEditWidgets(FMenuBuilder& MenuBuilder)
	{
		using namespace UE::ConcertFilterUtils;
		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TimeFilter.Reset.", "Reset"),
			LOCTEXT("TimeFilter.Reset.Tooltip", "Sets the time so that this filter has no effect"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ Implementation.ResetToInfiniteTime(); }),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked()),
			NAME_None,
			EUserInterfaceActionType::Button
			);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TimeFilter.Now.", "Now"),
			LOCTEXT("TimeFilter.Now.Tooltip", "Sets the time to now in local time"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ Implementation.SetTime(FDateTime::UtcNow()); }),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked()),
			NAME_None,
			EUserInterfaceActionType::Button
		);
		
		MenuBuilder.AddSeparator();

		// Year
		MenuBuilder.AddWidget(SetMenuWidgetWidth(CreateDateComponentNumericWidget(ETimeComponent::Year)), LOCTEXT("Year", "Year"), true);

		// Month
		using FMonth = int32;
		TRadialMenuBuilder<FMonth>::AddRadialSubMenu(
			MenuBuilder,
			LOCTEXT("Month", "Month"),
			TAttribute<FMonth>::CreateLambda([this](){ return Implementation.GetTime().GetMonth(); }),
			TArray<FMonth>{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12 },
			TRadialMenuBuilder<FMonth>::FItemToText::CreateLambda([](const FMonth& Month)
			{
				const FDateTime Date(1, Month, 1);
				return FText::FromString(Date.ToFormattedString(TEXT("%B")));
			}),
			TRadialMenuBuilder<FMonth>::FOnSelectItem::CreateLambda([this](const FMonth& Month)
			{
				const FDateTime CurrentTime = Implementation.GetTime();
				const int32 Day = FMath::Clamp(CurrentTime.GetDay(), 1, FDateTime::DaysInMonth(CurrentTime.GetYear(), Month));
				const FDateTime NewTime(CurrentTime.GetYear(), Month, Day, CurrentTime.GetHour(), CurrentTime.GetMinute(), CurrentTime.GetSecond());
				Implementation.SetTime(NewTime);
			})
			);

		// Day
		MenuBuilder.AddWidget(SetMenuWidgetWidth(CreateDateComponentNumericWidget(ETimeComponent::Day)), LOCTEXT("Day", "Day"), true);

		// Hour : Minute : Second
		const TSharedRef<SWidget> TimeWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				CreateDateComponentNumericWidget(ETimeComponent::Hour)
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Colon", ":"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				CreateDateComponentNumericWidget(ETimeComponent::Minute)
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Colon", ":"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				CreateDateComponentNumericWidget(ETimeComponent::Second)
			];
		MenuBuilder.AddWidget(SetMenuWidgetWidth(TimeWidget), LOCTEXT("Time", "Time"), true);
	}

	void FConcertFrontendLogFilter_Time::BuildMonthSubMenu(FMenuBuilder& MenuBuilder)
	{
		for (int32 MonthIdx = 1; MonthIdx <= 12; ++MonthIdx)
		{
			const FDateTime Date(1, MonthIdx, 1);
			MenuBuilder.AddMenuEntry(
				// B means long month name
				FText::FromString(Date.ToFormattedString(TEXT("%B"))),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, MonthIdx]()
					{
						const FDateTime CurrentTime = Implementation.GetTime();
						const FDateTime NewTime(CurrentTime.GetYear(), MonthIdx, CurrentTime.GetDay(), CurrentTime.GetHour(), CurrentTime.GetMinute(), CurrentTime.GetSecond());
						Implementation.SetTime(NewTime);
					}),
					FCanExecuteAction::CreateLambda([] { return true; }),
					FIsActionChecked::CreateLambda([this, MonthIdx](){ return Implementation.GetTime().GetMonth() == MonthIdx; })),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	TSharedRef<SWidget> FConcertFrontendLogFilter_Time::CreateDateComponentNumericWidget(ETimeComponent TimeComponent)
	{
		const uint32 Min = [TimeComponent]()
		{
			switch (TimeComponent)
			{
			case ETimeComponent::Year: return 1;
			case ETimeComponent::Day: return 1;
			case ETimeComponent::Hour: return 0;
			case ETimeComponent::Minute: return 0;
			case ETimeComponent::Second: return 0;
			default: checkNoEntry(); return 0;;
			}
		}();
		const uint32 Max = [TimeComponent]()
		{
			switch (TimeComponent)
			{
			case ETimeComponent::Year: return 9999;
			case ETimeComponent::Day: return 31;
			case ETimeComponent::Hour: return 23;
			case ETimeComponent::Minute: return 59;
			case ETimeComponent::Second: return 59;
			default: checkNoEntry(); return 0;;
			}
		}();

		auto MaxLambda = [this, TimeComponent, Max]() -> uint32
		{
			if (TimeComponent == ETimeComponent::Day)
			{
				const FDateTime Time = Implementation.GetTime();
				return FDateTime::DaysInMonth(Time.GetYear(), Time.GetMonth());
			}
			return Max;
		};
		
		return SNew(SNumericEntryBox<uint32>)
			.AllowSpin(true)
			.MinDesiredValueWidth(Min)
			.MinValue(Min)
			.MinSliderValue(Min)
			.MaxSliderValue_Lambda(MaxLambda)
			.MaxValue_Lambda(MaxLambda)
			.OnValueChanged_Lambda([this, TimeComponent](uint32 NewValue)
			{
				const FDateTime CurrentTime = Implementation.GetTime();
				switch (TimeComponent)
				{
				case ETimeComponent::Year: Implementation.SetTime({ static_cast<int32>(NewValue), CurrentTime.GetMonth(), CurrentTime.GetDay(), CurrentTime.GetHour(), CurrentTime.GetMinute(), CurrentTime.GetSecond() }); break;
				case ETimeComponent::Day: Implementation.SetTime({ CurrentTime.GetYear(), CurrentTime.GetMonth(), static_cast<int32>(NewValue), CurrentTime.GetHour(), CurrentTime.GetMinute(), CurrentTime.GetSecond() }); break;
				case ETimeComponent::Hour: Implementation.SetTime({ CurrentTime.GetYear(), CurrentTime.GetMonth(), CurrentTime.GetDay(), static_cast<int32>(NewValue), CurrentTime.GetMinute(), CurrentTime.GetSecond() }); break;
				case ETimeComponent::Minute: Implementation.SetTime({ CurrentTime.GetYear(), CurrentTime.GetMonth(), CurrentTime.GetDay(), CurrentTime.GetHour(), static_cast<int32>(NewValue), CurrentTime.GetSecond() }); break;
				case ETimeComponent::Second: Implementation.SetTime({ CurrentTime.GetYear(), CurrentTime.GetMonth(), CurrentTime.GetDay(), CurrentTime.GetHour(), CurrentTime.GetMinute(), static_cast<int32>(NewValue) }); break;
				default: ;
				}
			})
			.Value_Lambda([this, TimeComponent]()
			{
				switch (TimeComponent)
				{
				case ETimeComponent::Year: return Implementation.GetTime().GetYear();
				case ETimeComponent::Day: return Implementation.GetTime().GetDay();
				case ETimeComponent::Hour: return Implementation.GetTime().GetHour();
				case ETimeComponent::Minute: return Implementation.GetTime().GetMinute();
				case ETimeComponent::Second: return Implementation.GetTime().GetSecond();
				default: return 0;
				}
			});
	}
}

#undef LOCTEXT_NAMESPACE
