// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Util/Filter/ConcertFrontendFilter.h"

struct FConcertLogEntry;

namespace UE::MultiUserServer
{
	enum class ETimeFilter
	{
		/** Logs after the indicated time are allowed */
		AllowAfter,
		/** Logs before the indicated time are allowed */
		AllowBefore
	};

	/** Filters based on whether a log happened before or after a certain time */
	class FConcertLogFilter_Time : public IFilter<const FConcertLogEntry&>
	{
	public:

		FConcertLogFilter_Time(ETimeFilter FilterMode);

		void ResetToInfiniteTime();

		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(const FConcertLogEntry& InItem) const override;
		virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
		//~ End FConcertLogFilter Interfac

		ETimeFilter GetFilterMode() const { return FilterMode; }
		FDateTime GetTime() const { return Time; }

		void SetFilterMode(ETimeFilter InFilterMode);
		void SetTime(const FDateTime& InTime);
	
	private:

		ETimeFilter FilterMode;
		FDateTime Time;
		FChangedEvent ChangedEvent;

		FDateTime MakeResetTime() const;
	};

	class FConcertFrontendLogFilter_Time : public TConcertFrontendFilterAggregate<FConcertLogFilter_Time, const FConcertLogEntry&>
	{
		using Super = TConcertFrontendFilterAggregate<FConcertLogFilter_Time, const FConcertLogEntry&>;
	public:
	
		FConcertFrontendLogFilter_Time(TSharedRef<FFilterCategory> FilterCategory, ETimeFilter TimeFilter);
		
		virtual FString GetName() const override { return TimeFilter == ETimeFilter::AllowAfter ? TEXT("After") : TEXT("Before"); }
		virtual FText GetDisplayName() const override
		{
			return TimeFilter == ETimeFilter::AllowAfter
				? NSLOCTEXT("UnrealMultiUserUI.FConcertFrontendLogFilter_Time", "DisplayLabel.After", "After Timestamp")
				: NSLOCTEXT("UnrealMultiUserUI.FConcertFrontendLogFilter_Time", "DisplayLabel.Before", "Before Timestamp");
		}

		virtual void ExposeEditWidgets(FMenuBuilder& MenuBuilder) override;

	private:

		enum class ETimeComponent
		{
			Year,
			Day,
			Hour,
			Minute,
			Second
		};
		
		ETimeFilter TimeFilter;
		
		void BuildMonthSubMenu(FMenuBuilder& MenuBuilder);
		TSharedRef<SWidget> CreateDateComponentNumericWidget(ETimeComponent TimeComponent);
	};
}

