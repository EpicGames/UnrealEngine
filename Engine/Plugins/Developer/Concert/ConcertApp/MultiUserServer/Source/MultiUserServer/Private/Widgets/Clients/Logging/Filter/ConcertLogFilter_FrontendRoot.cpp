// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogFilter_FrontendRoot.h"

#include "ConcertFrontendLogFilter_Ack.h"
#include "ConcertFrontendLogFilter_Client.h"
#include "ConcertFrontendLogFilter_MessageAction.h"
#include "ConcertFrontendLogFilter_MessageType.h"
#include "ConcertFrontendLogFilter_Time.h"
#include "ConcertFrontendLogFilter_Size.h"
#include "Widgets/Clients/Logging/Filter/ConcertFrontendLogFilter_BaseSetSelection.h"
#include "Widgets/Clients/Logging/Filter/ConcertFrontendLogFilter_TextSearch.h"


namespace UE::MultiUserServer
{
	namespace Private
	{
		static TArray<TSharedRef<TConcertFrontendFilter<const FConcertLogEntry&>>> CreateCommonFilters(const TSharedRef<FFilterCategory>& FilterCategory)
		{
			return {
				MakeShared<FConcertFrontendLogFilter_MessageAction>(FilterCategory),
				MakeShared<FConcertFrontendLogFilter_MessageType>(FilterCategory),
				MakeShared<FConcertFrontendLogFilter_Time>(FilterCategory, ETimeFilter::AllowAfter),
				MakeShared<FConcertFrontendLogFilter_Time>(FilterCategory, ETimeFilter::AllowBefore),
				MakeShared<FConcertFrontendLogFilter_Size>(FilterCategory),
				MakeShared<FConcertFrontendLogFilter_Ack>(FilterCategory)
			};
		}
	}
	
	TSharedRef<FConcertLogFilter_FrontendRoot> MakeGlobalLogFilter(TSharedRef<FConcertLogTokenizer> Tokenizer)
	{
		TSharedRef<FFilterCategory> FilterCategory = MakeShared<FFilterCategory>(
			NSLOCTEXT("UnrealMultiUserUI.TConcertFrontendRootFilter", "DefaultCategoryLabel", "Default"),
			FText::GetEmpty()
			);
		return MakeShared<FConcertLogFilter_FrontendRoot>(
			MoveTemp(Tokenizer),
			Private::CreateCommonFilters(FilterCategory),
			FilterCategory
			);
	}

	TSharedRef<FConcertLogFilter_FrontendRoot> MakeClientLogFilter(TSharedRef<FConcertLogTokenizer> Tokenizer, const FGuid& ClientMessageNodeId, const TSharedRef<FEndpointToUserNameCache>& EndpointCache)
	{
		TSharedRef<FFilterCategory> FilterCategory = MakeShared<FFilterCategory>(
			NSLOCTEXT("UnrealMultiUserUI.TConcertFrontendRootFilter", "DefaultCategoryLabel", "Default"),
			FText::GetEmpty()
			);
		const TArray<TSharedRef<TConcertFrontendFilter<const FConcertLogEntry&>>> CommonFilters = Private::CreateCommonFilters(FilterCategory);
		const TArray<TSharedRef<IFilter<const FConcertLogEntry&>>> NonVisuals = {
			MakeShared<FConcertLogFilter_Client>(ClientMessageNodeId, EndpointCache)
		};
		return MakeShared<FConcertLogFilter_FrontendRoot>(
			MoveTemp(Tokenizer),
			CommonFilters,
			MoveTemp(FilterCategory),
			NonVisuals
			);
	}
}
