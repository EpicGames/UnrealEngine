// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendLogFilter_TextSearch.h"
#include "Widgets/Util/Filter/ConcertRootFrontendFilter.h"
#include "Widgets/Util/Filter/SConcertFilterBar.h"

namespace UE::MultiUserServer { class FConcertLogTokenizer; }

class FConcertLogTokenizer;
class FEndpointToUserNameCache;
struct FConcertLogEntry;

namespace UE::MultiUserServer
{
	/** A filter that contains multiple UI filters */
	class FConcertLogFilter_FrontendRoot : public TConcertFrontendRootFilter<const FConcertLogEntry&, FConcertFrontendLogFilter_TextSearch>
	{
		using Super = TConcertFrontendRootFilter<const FConcertLogEntry&, FConcertFrontendLogFilter_TextSearch>;
	public:

		FConcertLogFilter_FrontendRoot(
			TSharedRef<FConcertLogTokenizer> Tokenizer,
			TArray<TSharedRef<TConcertFrontendFilter<const FConcertLogEntry&>>> FrontendFilters,
			TSharedRef<FFilterCategory> DefaultCategory,
			const TArray<TSharedRef<IFilter<const FConcertLogEntry&>>>& NonVisualFilters = {}
			)
			: Super(MakeShared<FConcertFrontendLogFilter_TextSearch>(DefaultCategory, MoveTemp(Tokenizer)), MoveTemp(FrontendFilters), NonVisualFilters, DefaultCategory)
		{}
	};
	
	/** Creates a filter for the global filter log window */
	TSharedRef<FConcertLogFilter_FrontendRoot> MakeGlobalLogFilter(TSharedRef<FConcertLogTokenizer> Tokenizer);

	/**
	 * Creates a filter for a filter log window intended for a client
	 * @param Tokenizer Used for text search
	 * @param ClientMessageNodeId The Id of this client's messaging node - this is used to filter messages involving this client
	 * @param EndpointCache Needed to filter messages involving this client - converts Concert endpoint Ids to the message node Id
	 */
	TSharedRef<FConcertLogFilter_FrontendRoot> MakeClientLogFilter(TSharedRef<FConcertLogTokenizer> Tokenizer, const FGuid& ClientMessageNodeId, const TSharedRef<FEndpointToUserNameCache>& EndpointCache);
}
