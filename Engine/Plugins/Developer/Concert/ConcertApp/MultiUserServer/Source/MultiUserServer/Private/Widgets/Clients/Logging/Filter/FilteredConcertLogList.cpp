// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilteredConcertLogList.h"


namespace UE::MultiUserServer
{
	FFilteredConcertLogList::FFilteredConcertLogList(TSharedRef<IConcertLogSource> InLogSource, TSharedPtr<IFilter<const FConcertLogEntry&>> InOptionalFilter)
		: LogSource(MoveTemp(InLogSource))
		, Filter(MoveTemp(InOptionalFilter))
	{
		LogSource->OnLowestLogEntryChanged().AddRaw(this, &FFilteredConcertLogList::OnLowestLogEntryChanged);
		LogSource->OnLogEntryAdded().AddRaw(this, &FFilteredConcertLogList::OnNewLogEntryAdded);

		if (Filter)
		{
			Filter->OnChanged().AddRaw(this, &FFilteredConcertLogList::RebuildFilteredResult);
		}

		RebuildFilteredResult();
	}

	FFilteredConcertLogList::~FFilteredConcertLogList()
	{
		// This is not really needed because the LogSource is SUPPOSED to be solely owned by FFilteredConcertLogList.
		// We do it for safety regardless...
		LogSource->OnLowestLogEntryChanged().RemoveAll(this);
		LogSource->OnLogEntryAdded().RemoveAll(this);

		// ... Filter is not uniquely owned but removing effectively also does not really matter.
		if (Filter)
		{
			Filter->OnChanged().RemoveAll(this);
		}
	}

	void FFilteredConcertLogList::OnLowestLogEntryChanged(FConcertLogID NewLowestValidID)
	{
		for (size_t i = 0; i < FilteredLogs.Num() && FilteredLogs[i]->LogId < NewLowestValidID; ++i)
		{
			// Worst case this is O(n^2) - usually a thought O(cn) with small c)...
			// Not much we without rewriting SListView to use something other than TArray
			RemoveMessageIdToLogEntry(FilteredLogs[i]);
			FilteredLogs.RemoveAt(i);
			--i;
		}
	}

	void FFilteredConcertLogList::OnNewLogEntryAdded(const TSharedRef<FConcertLogEntry>& NewLogEntry)
	{
		if (!Filter || Filter->PassesFilter(*NewLogEntry))
		{
			MessageIdToLogs.FindOrAdd(NewLogEntry->Log.MessageId).AddUnique(NewLogEntry);
			FilteredLogs.Emplace(NewLogEntry);
			OnLogListChanged().Broadcast(GetFilteredLogs());
		}
	}

	void FFilteredConcertLogList::RebuildFilteredResult()
	{
		MessageIdToLogs.Reset();
		FilteredLogs.Reset();
		LogSource->ForEachLog([this](const TSharedPtr<FConcertLogEntry>& LogEntry)
		{
			if (!Filter || Filter->PassesFilter(*LogEntry))
			{
				MessageIdToLogs.FindOrAdd(LogEntry->Log.MessageId).AddUnique(LogEntry);
				FilteredLogs.Emplace(LogEntry);
			}
		});

		OnLogListChanged().Broadcast(GetFilteredLogs());
	}

	void FFilteredConcertLogList::RemoveMessageIdToLogEntry(const TSharedPtr<FConcertLogEntry>& Log)
	{
		TArray<TSharedPtr<FConcertLogEntry>>* Entries = MessageIdToLogs.Find(Log->Log.MessageId);
		if (!Entries)
		{
			return;
		}
		
		if (Entries->Num() == 1)
		{
			MessageIdToLogs.Remove(Log->Log.MessageId);
		}
		else
		{
			Entries->RemoveSingle(Log);
		}
	}

	FPagedFilteredConcertLogList::FPagedFilteredConcertLogList(TSharedRef<IConcertLogSource> LogSource, TSharedPtr<IFilter<const FConcertLogEntry&>> OptionalFilter, uint16 InitialLogsPerPage)
		: FFilteredConcertLogList(MoveTemp(LogSource), MoveTemp(OptionalFilter))
		, LogsPerPage(FMath::Max<uint16>(InitialLogsPerPage, 1))
	{
		checkf(InitialLogsPerPage >= 1, TEXT("Unreasonable page size"));

		OnLogListChanged().AddRaw(this, &FPagedFilteredConcertLogList::HandeOnLogListChanged);
		CheckAndConditionallyPopulatePage();
	}

	TOptional<FPagedFilteredConcertLogList::FPageCount> FPagedFilteredConcertLogList::GetPageOf(const size_t ItemIndex) const
	{
		const FPageCount Page = ItemIndex / GetLogsPerPage();
		if (Page < GetNumPages())
		{
			return Page;
		}
		return {};
	}

	void FPagedFilteredConcertLogList::SetLogsPerPage(uint16 NewLogsPerPage)
	{
		if (LogsPerPage != NewLogsPerPage && ensure(NewLogsPerPage > 0))
		{
			size_t StartIndex, LastIndex;
			Tie(StartIndex, LastIndex) = GetLogIndicesForPage();
			
			LogsPerPage = NewLogsPerPage;
			// Recalculate current page so the first item that was the page is still on the page
			CurrentPageIndex = StartIndex / LogsPerPage;

			RepopulatePage();
		}
	}

	void FPagedFilteredConcertLogList::SetPage(const uint32 PageIndex)
	{
		if (CurrentPageIndex != PageIndex && PageIndex < GetNumPages())
		{
			CurrentPageIndex = PageIndex;
			RepopulatePage();
		}
	}

	void FPagedFilteredConcertLogList::HandeOnLogListChanged(const TArray<TSharedPtr<FConcertLogEntry>>& NewFilteredLogList)
	{
		// After filtering there may be less logs > less pages > the current page might have become out of bounds
		if (CurrentPageIndex >= GetNumPages())
		{
			CurrentPageIndex = 0;
			RepopulatePage();
		}
		else
		{
			CheckAndConditionallyPopulatePage();
		}
	}

	void FPagedFilteredConcertLogList::RepopulatePage()
	{
		PageView.Empty(LogsPerPage);
		ForEachLogIndexOnPage([this](size_t i)
		{
			PageView.Emplace(GetFilteredLogs()[i]);
		});

		OnPageViewChanged().Broadcast(PageView);
	}

	void FPagedFilteredConcertLogList::CheckAndConditionallyPopulatePage()
	{
		bool bChanged = false;
		size_t PageViewIndex = 0;
		ForEachLogIndexOnPage([this, &PageViewIndex, &bChanged](size_t i)
		{
			if (PageView[PageViewIndex] != GetFilteredLogs()[i])
			{
				PageView[PageViewIndex] = GetFilteredLogs()[i];
				bChanged = true;
			}

			++PageViewIndex;
		}, FMath::Min(PageView.Num(), GetFilteredLogs().Num()));

		size_t StartIndex, ItemsOnPage;
		Tie(StartIndex, ItemsOnPage) = GetLogIndicesForPage();

		// Add items that are new in GetFilteredLogs()
		bChanged |= PageView.Num() < ItemsOnPage; 
		for (size_t i = PageView.Num(); i < ItemsOnPage && GetFilteredLogs().IsValidIndex(i); ++i) //-V614
		{
			PageView.Emplace(GetFilteredLogs()[i]);
		}

		// Remove (possibly duplicate) items that exceed the expected page length 
		const size_t NumToRemove = PageView.Num() - ItemsOnPage;
		bChanged |= NumToRemove > 0;
		for (size_t i = 0; i < NumToRemove; ++i)
		{
			PageView.RemoveAt(PageView.Num() - 1);
		}

		if (bChanged)
		{
			OnPageViewChanged().Broadcast(PageView);
		}
	}

	void FPagedFilteredConcertLogList::ForEachLogIndexOnPage(TFunctionRef<void(size_t Index)> Callback, FLogsPerPageCount MaxItems)
	{
		size_t StartIndex, NumItems;
		Tie(StartIndex, NumItems) = GetLogIndicesForPage();
		for (size_t i = 0; i < NumItems && i < MaxItems; ++i) //-V614
		{
			Callback(StartIndex + i);
		}
	}

	TTuple<size_t, size_t> FPagedFilteredConcertLogList::GetLogIndicesForPage() const
	{
		const size_t StartIndex = CurrentPageIndex * LogsPerPage;
		const size_t NumberItems = FMath::Min<size_t>(
			// Used when page is full
			LogsPerPage,
			// Used when last page is not full
			GetFilteredLogs().Num() - StartIndex
			);
		check(GetFilteredLogs().Num() == 0 || GetFilteredLogs().IsValidIndex(StartIndex));
		return { StartIndex, NumberItems };
	}
}
