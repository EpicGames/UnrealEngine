// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/IFilter.h"
#include "Widgets/Clients/Logging/Source/IConcertLogSource.h"

template <typename OptionalType> struct TOptional;

struct FConcertLog;

namespace UE::MultiUserServer
{
	/** Applies a filter to a IConcertLogSource as the logs come in. */
	class FFilteredConcertLogList
	{
	public:
		
		FFilteredConcertLogList(TSharedRef<IConcertLogSource> InLogSource, TSharedPtr<IFilter<const FConcertLogEntry&>> InOptionalFilter);
		~FFilteredConcertLogList();
		
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnLogListChanged, const TArray<TSharedPtr<FConcertLogEntry>>& /*NewFilteredView*/);
		FOnLogListChanged& OnLogListChanged() { return OnLogListChangedEvent; }

		bool IsBufferFull() const { return LogSource->IsBufferFull(); }
		const TArray<TSharedPtr<FConcertLogEntry>>& GetFilteredLogs() const { return FilteredLogs; }
		TArray<TSharedPtr<FConcertLogEntry>> GetFilteredLogsWithId(const FGuid& MessageId) const
		{
			const TArray<TSharedPtr<FConcertLogEntry>>* Result = MessageIdToLogs.Find(MessageId);
			return Result ? *Result : TArray<TSharedPtr<FConcertLogEntry>>{};
		}

	protected:

		const TSharedRef<IConcertLogSource>& GetLogSource() const { return LogSource; }
		
	private:

		/** Determines where the logs come from */
		TSharedRef<IConcertLogSource> LogSource;

		/** Optional. Filters items from the LogSource */
		TSharedPtr<IFilter<const FConcertLogEntry&>> Filter;

		/** Contains the items of LogSource with Filter applied. */
		TArray<TSharedPtr<FConcertLogEntry>> FilteredLogs;

		/** Binds message IDs to the logs that have that ID. Logs can share the same ID but have different MessageActions. */
		TMap<FGuid, TArray<TSharedPtr<FConcertLogEntry>>> MessageIdToLogs;
		
		/** Called when the content of FilteredResult changes */
		FOnLogListChanged OnLogListChangedEvent;

		void OnLowestLogEntryChanged(FConcertLogID NewLowestValidID);
		void OnNewLogEntryAdded(const TSharedRef<FConcertLogEntry>& NewLogEntry);
		void RebuildFilteredResult();

		void RemoveMessageIdToLogEntry(const TSharedPtr<FConcertLogEntry>& Log);
	};

	/** Further sorts the results of FFilteredConcertLogList::Filter into pages, i.e. buckets of logs, useful for limiting elements in a table view. */
	class FPagedFilteredConcertLogList : public FFilteredConcertLogList
	{
	public:
		
		using FLogsPerPageCount = uint16;
		using FPageCount = uint32;

		FPagedFilteredConcertLogList(TSharedRef<IConcertLogSource> LogSource, TSharedPtr<IFilter<const FConcertLogEntry&>> OptionalFilter, uint16 FLogsPerPageCount = 500);

		FPageCount GetNumPages() const { const FPageCount PageCount = GetFilteredLogs().Num() / LogsPerPage; return GetFilteredLogs().Num() % LogsPerPage == 0 ? PageCount : PageCount + 1;}
		FPageCount GetCurrentPage() const { return CurrentPageIndex; }
		TOptional<FPageCount> GetPageOf(const size_t ItemIndex) const;
		FLogsPerPageCount GetLogsPerPage() const { return LogsPerPage; }
		const TArray<TSharedPtr<FConcertLogEntry>>& GetPageView() const { return PageView; }
		
		void SetLogsPerPage(FLogsPerPageCount NewLogsPerPage);
		void SetPage(const FPageCount PageIndex);

		FOnLogListChanged& OnPageViewChanged() { return OnPageViewChangedEvent; }
		
	private:


		/** The number of logs to show per page*/
		FLogsPerPageCount LogsPerPage;

		/** The index of the current page. Starts at 0. */
		FPageCount CurrentPageIndex = 0;

		/** The filtered logs on the current page */
		TArray<TSharedPtr<FConcertLogEntry>> PageView;

		/** Called when the content of PageView changes */
		FOnLogListChanged OnPageViewChangedEvent;

		void HandeOnLogListChanged(const TArray<TSharedPtr<FConcertLogEntry>>& NewFilteredLogList);
		void RepopulatePage();
		void CheckAndConditionallyPopulatePage();
		
		/**
		 * Given the current page settings (not validated - assumed corrected), calls Callback with each index in GetFilteredLogs()
		 * that should be an item on the current page. */
		void ForEachLogIndexOnPage(TFunctionRef<void(size_t FilteredLogIndex)> Callback, FLogsPerPageCount MaxItems = TNumericLimits<FLogsPerPageCount>::Max());
		/** @return Start and last index in GetFilteredLogs() array to accommodate current page settings */
		TTuple<size_t, size_t> GetLogIndicesForPage() const;
	};
}
