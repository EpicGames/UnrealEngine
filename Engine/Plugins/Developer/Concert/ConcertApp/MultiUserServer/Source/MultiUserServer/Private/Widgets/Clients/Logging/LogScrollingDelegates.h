// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Clients/Logging/ConcertLogEntry.h"

namespace UE::MultiUserServer
{
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCanScrollToLog, const FGuid& /*MessageId*/, FConcertLogEntryFilterFunc /*Filter*/, FText& /*ErrorMessage*/);
	DECLARE_DELEGATE_TwoParams(FScrollToLog, const FGuid& /*MessageId*/, FConcertLogEntryFilterFunc /*Filter*/);
}
