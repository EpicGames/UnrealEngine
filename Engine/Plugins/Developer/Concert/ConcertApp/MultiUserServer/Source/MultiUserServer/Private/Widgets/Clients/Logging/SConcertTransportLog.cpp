// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertTransportLog.h"

#include "Filter/ConcertLogFilter_FrontendRoot.h"
#include "Filter/ConcertFrontendLogFilter_TextSearch.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Util/ConcertLogTokenizer.h"
#include "SConcertTransportLogFooter.h"
#include "SConcertTransportLogRow.h"
#include "Settings/ConcertTransportLogSettings.h"
#include "Settings/MultiUserServerColumnVisibilitySettings.h"
#include "Widgets/Clients/Logging/Filter/FilteredConcertLogList.h"
#include "Widgets/Clients/SPromptConcertLoggingEnabled.h"

#include "Widgets/Clients/Util/EndpointToUserNameCache.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertTransportLog"

namespace UE::MultiUserServer
{
	const FName SConcertTransportLog::FirstColumnId("AvatarColourColumnId");
	
	SConcertTransportLog::~SConcertTransportLog()
	{
		ConcertTransportEvents::OnConcertTransportLoggingEnabledChangedEvent().RemoveAll(this);
		UMultiUserServerColumnVisibilitySettings::GetSettings()->OnTransportLogColumnVisibility().RemoveAll(this);
	}

	void SConcertTransportLog::Construct(const FArguments& InArgs, TSharedRef<IConcertLogSource> LogSource, TSharedRef<FEndpointToUserNameCache> InEndpointCache, TSharedRef<FConcertLogTokenizer> InLogTokenizer)
	{
		PagedLogList = MakeShared<FPagedFilteredConcertLogList>(MoveTemp(LogSource), InArgs._Filter);
		EndpointCache = MoveTemp(InEndpointCache);
		LogTokenizer = MoveTemp(InLogTokenizer);
		Filter = InArgs._Filter;
		
		ChildSlot
		[
			SAssignNew(EnableLoggingPromptOverlay, SOverlay)

			+SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.BorderBackgroundColor(FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f)))
				.Padding(2)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Top)
					[
						InArgs._Filter
							? InArgs._Filter->BuildFilterWidgets
							(
								FFilterWidgetArgs()
								.RightOfSearchBar
								(
									ConcertFrontendUtils::CreateViewOptionsComboButton(FOnGetContent::CreateSP(this, &SConcertTransportLog::CreateViewOptionsMenu))
								)
							)
							: SNullWidget::NullWidget
					]

					+SVerticalBox::Slot()
					.FillHeight(1.f)
					.Padding(0, 5, 0, 0)
					[
						CreateTableView()
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SConcertTransportLogFooter, PagedLogList.ToSharedRef())
					]
				]
			]
		];

		// Subscribe to events
		PagedLogList->OnPageViewChanged().AddSP(this, &SConcertTransportLog::OnPageViewChanged);
		
		UMultiUserServerColumnVisibilitySettings::GetSettings()->OnTransportLogColumnVisibility().AddSP(this, &SConcertTransportLog::OnColumnVisibilitySettingsChanged);
		ConcertSharedSlate::RestoreColumnVisibilityState(HeaderRow.ToSharedRef(), UMultiUserServerColumnVisibilitySettings::GetSettings()->GetTransportLogColumnVisibility());
		
		ConcertTransportEvents::OnConcertTransportLoggingEnabledChangedEvent().AddSP(this, &SConcertTransportLog::OnConcertLoggingEnabledChanged);
		OnConcertLoggingEnabledChanged(ConcertTransportEvents::IsLoggingEnabled());
	}

	bool SConcertTransportLog::CanScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc) const
	{
		return PagedLogList->GetFilteredLogsWithId(MessageId).ContainsByPredicate([FilterFunc](const TSharedPtr<FConcertLogEntry>& LogEntry)
		{
			return FilterFunc(*LogEntry);
		});
	}

	void SConcertTransportLog::ScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc) const
	{
		// Could be optimised further by having FPagedFilteredConcertLogList cache the index but it's fine - ScrollToLog is called very infrequently (when the user presses a button)
		const int32 Index = PagedLogList->GetFilteredLogs().IndexOfByPredicate([MessageId, FilterFunc](const TSharedPtr<FConcertLogEntry>& Entry)
		{
			return Entry->Log.MessageId == MessageId && FilterFunc(*Entry);
		});
		ScrollToLog(Index);
	}

	TSharedRef<SWidget> SConcertTransportLog::CreateTableView()
	{
		CreateHeaderRow();
		return SAssignNew(LogView, SListView<TSharedPtr<FConcertLogEntry>>)
			.ListItemsSource(&PagedLogList->GetPageView())
			.OnGenerateRow(this, &SConcertTransportLog::OnGenerateActivityRowWidget)
			.OnContextMenuOpening_Lambda([this](){ return ConcertSharedSlate::MakeTableContextMenu(HeaderRow.ToSharedRef(), GetDefaultColumnVisibilities(), true); })
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow(HeaderRow);
	}

	TSharedRef<SHeaderRow> SConcertTransportLog::CreateHeaderRow()
	{
		HeaderRow = SNew(SHeaderRow)
			.OnHiddenColumnsListChanged_Lambda([this]()
			{
				if (!bIsUpdatingColumnVisibility)
				{
					UMultiUserServerColumnVisibilitySettings::GetSettings()->SetTransportLogColumnVisibility(
					UE::ConcertSharedSlate::SnapshotColumnVisibilityState(HeaderRow.ToSharedRef())
					);
				}
			})
			// Create tiny dummy row showing avatar colour to handle the case when a user hides all columns
			+SHeaderRow::Column(FirstColumnId)
				.DefaultLabel(FText::GetEmpty())
				.FixedWidth(8)
				// Cannot be hidden
				.ShouldGenerateWidget(true)
				.ToolTipText(LOCTEXT("AvatarColumnToolTipText", "The colour of the avatar is affected by log"))
			;
		
		ForEachPropertyColumn([this](const FProperty& Property, FName ColumnId)
		{
			const TMap<FName, FString> ColumnNameOverrides = {
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageAction), TEXT("Action") },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageOrderIndex), TEXT("Order Index") },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, ChannelId), TEXT("Channel") },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, CustomPayloadUncompressedByteSize), TEXT("Size") },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, OriginEndpointId), TEXT("Origin") },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, DestinationEndpointId), TEXT("Destination") },
				{ GET_MEMBER_NAME_CHECKED(FConcertLogMetadata, AckState), TEXT("Ack") }
			};
			const TMap<FName, float> ManuallySizeColumns = {
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, Frame), 60 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageId), 280 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageOrderIndex), 60 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, ChannelId), 60 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, Timestamp), 160 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageAction), 80 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageTypeName), 200 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, CustomPayloadTypename), 400 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageAction), 80 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLog, CustomPayloadUncompressedByteSize), 100 },
				{ GET_MEMBER_NAME_CHECKED(FConcertLogMetadata, AckState), 45 }
			};
			
			const FString* LabelOverride = ColumnNameOverrides.Find(ColumnId);
			const FString PropertyName = LabelOverride ? *LabelOverride : Property.GetAuthoredName();
			SHeaderRow::FColumn::FArguments Args = SHeaderRow::FColumn::FArguments()
				.ColumnId(ColumnId)
				.DefaultLabel(FText::FromString(PropertyName))
				.HAlignCell(HAlign_Center);
			
			if (const float* ManualSize = ManuallySizeColumns.Find(ColumnId))
			{
				Args.ManualWidth(*ManualSize);
			}
			
			HeaderRow->AddColumn(Args);
		});

		TGuardValue<bool> DoNotSave(bIsUpdatingColumnVisibility, true);
		RestoreDefaultColumnVisiblities();
		
		return HeaderRow.ToSharedRef();
	}

	TSharedRef<ITableRow> SConcertTransportLog::OnGenerateActivityRowWidget(TSharedPtr<FConcertLogEntry> Item, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		const TOptional<FConcertClientInfo> OriginInfo = EndpointCache->GetClientInfo(Item->Log.OriginEndpointId);
		const TOptional<FConcertClientInfo> DestinationInfo = EndpointCache->GetClientInfo(Item->Log.DestinationEndpointId);
		const FLinearColor AvatarColor = OriginInfo.IsSet()
			? OriginInfo->AvatarColor
			: DestinationInfo.IsSet() ? DestinationInfo->AvatarColor : FLinearColor::Black;
		
		return SNew(SConcertTransportLogRow, Item, OwnerTable, LogTokenizer.ToSharedRef())
			.HighlightText_Lambda([this](){ return Filter.IsValid() ? Filter->GetTextSearchFilter()->GetSearchText() : FText(); })
			.AvatarColor(AvatarColor)
			.CanScrollToAckLog(this, &SConcertTransportLog::CanScrollToAckLog)
			.CanScrollToAckedLog(this, &SConcertTransportLog::CanScrollToAckedLog)
			.ScrollToAckLog(this, &SConcertTransportLog::ScrollToAckLog)
			.ScrollToAckedLog(this, &SConcertTransportLog::ScrollToAckedLog);
	}

	void SConcertTransportLog::ForEachPropertyColumn(TFunctionRef<void(const FProperty& ColumnPropertyId, FName ColumnId)> Callback)
	{
		for (TFieldIterator<const FProperty> PropertyIt(FConcertLog::StaticStruct()); PropertyIt; ++PropertyIt)
		{
			if (!PropertyIt->HasAnyPropertyFlags(CPF_Transient))
			{
				Callback(**PropertyIt, PropertyIt->GetFName());
			}
		}
		for (TFieldIterator<const FProperty> PropertyIt(FConcertLogMetadata::StaticStruct()); PropertyIt; ++PropertyIt)
		{
			Callback(**PropertyIt, PropertyIt->GetFName());
		}
	}

	void SConcertTransportLog::RestoreDefaultColumnVisiblities()
	{
		const TMap<FName, bool> DefaultVisibilities = GetDefaultColumnVisibilities();
		ForEachPropertyColumn([this, &DefaultVisibilities](const FProperty& Property, FName ColumnId)
		{
			const bool* Visibility = DefaultVisibilities.Find(ColumnId);
			HeaderRow->SetShowGeneratedColumn(ColumnId, Visibility ? *Visibility : true);
		});
	}

	TMap<FName, bool> SConcertTransportLog::GetDefaultColumnVisibilities() const
	{
		return {
			{ GET_MEMBER_NAME_CHECKED(FConcertLog, Frame), false },
			{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageId), false },
			{ GET_MEMBER_NAME_CHECKED(FConcertLog, MessageOrderIndex), false },
			{ GET_MEMBER_NAME_CHECKED(FConcertLog, ChannelId), false },
			{ GET_MEMBER_NAME_CHECKED(FConcertLog, CustomPayloadTypename), false },
			{ GET_MEMBER_NAME_CHECKED(FConcertLog, StringPayload), false}
		};
	}

	TSharedRef<SWidget> SConcertTransportLog::CreateViewOptionsMenu()
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScroll", "Auto Scroll"),
			LOCTEXT("AutoScroll_Tooltip", "Automatically scroll as new logs arrive (affects last page)"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ bAutoScroll = !bAutoScroll; }),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([this] { return bAutoScroll; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
				LOCTEXT("DisplayTimestampInRelativeTime", "Display Relative Time"),
				TAttribute<FText>::CreateLambda([this]()
				{
					const bool bIsVisible = HeaderRow->IsColumnVisible(GET_MEMBER_NAME_CHECKED(FConcertLog, Timestamp));
					return bIsVisible
						? LOCTEXT("DisplayTimestampInRelativeTime.Tooltip.Visible", "Display the Last Modified column in relative time?")
						: LOCTEXT("DisplayTimestampInRelativeTime.Tooltip.Hidden", "Disabled because the Timestamp column is hidden.");
				}),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SConcertTransportLog::OnFilterMenuChecked),
					FCanExecuteAction::CreateLambda([this] { return HeaderRow->IsColumnVisible(GET_MEMBER_NAME_CHECKED(FConcertLog, Timestamp)); }),
					FIsActionChecked::CreateLambda([this] { return UConcertTransportLogSettings::GetSettings()->TimestampTimeFormat == ETimeFormat::Relative; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

		return MenuBuilder.MakeWidget();
	}

	void SConcertTransportLog::OnFilterMenuChecked()
	{
		UConcertTransportLogSettings* Settings = UConcertTransportLogSettings::GetSettings();
		
		switch (Settings->TimestampTimeFormat)
		{
		case ETimeFormat::Relative:
			Settings->TimestampTimeFormat = ETimeFormat::Absolute;
			break;
		case ETimeFormat::Absolute: 
			Settings->TimestampTimeFormat = ETimeFormat::Relative;
			break;
		default:
			checkNoEntry();
		}

		Settings->SaveConfig();
	}

	void SConcertTransportLog::OnPageViewChanged(const TArray<TSharedPtr<FConcertLogEntry>>&)
	{
		LogView->RequestListRefresh();

		if (bAutoScroll && PagedLogList->GetCurrentPage() == PagedLogList->GetNumPages() - 1)
		{
			LogView->ScrollToBottom();
		}
	}

	void SConcertTransportLog::OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot)
	{
		TGuardValue<bool> GuardValue(bIsUpdatingColumnVisibility, true);
		ConcertSharedSlate::RestoreColumnVisibilityState(HeaderRow.ToSharedRef(), ColumnSnapshot);
	}

	void SConcertTransportLog::OnConcertLoggingEnabledChanged(bool bNewEnabled)
	{
		if (!bNewEnabled)
		{
			EnableLoggingPromptOverlay->AddSlot().AttachWidget(SAssignNew(EnableLoggingPrompt, SPromptConcertLoggingEnabled));
		}
		else if (EnableLoggingPrompt)
		{
			EnableLoggingPromptOverlay->RemoveSlot(EnableLoggingPrompt.ToSharedRef());
		}
	}

	bool SConcertTransportLog::CanScrollToAckLog(const FGuid& MessageId) const
	{
		return CanScrollToLog(MessageId, [MessageId](const FConcertLogEntry& Entry)
		{
			return SharedCanScrollToAckLog(MessageId, Entry);
		});
	}

	bool SConcertTransportLog::CanScrollToAckedLog(const FGuid& MessageId) const
	{
		return CanScrollToLog(MessageId, [MessageId](const FConcertLogEntry& Entry)
		{
			return SharedCanScrollToAckedLog(MessageId, Entry);
		});
	}

	void SConcertTransportLog::ScrollToAckLog(const FGuid& MessageId) const
	{
		ScrollToLog(MessageId, [MessageId](const FConcertLogEntry& Entry)
		{
			return SharedCanScrollToAckLog(MessageId, Entry);
		});
	}

	void SConcertTransportLog::ScrollToAckedLog(const FGuid& MessageId) const
	{
		ScrollToLog(MessageId, [MessageId](const FConcertLogEntry& Entry)
		{
			return SharedCanScrollToAckedLog(MessageId, Entry);
		});
	}

	bool SConcertTransportLog::SharedCanScrollToAckLog(const FGuid& MessageId, const FConcertLogEntry& Entry)
	{
		// MessageId is shared by some logs: there is a log for receiving one one for processing for example.
		return Entry.Log.MessageId == MessageId && Entry.Log.MessageAction == EConcertLogMessageAction::Process;
	}

	bool SConcertTransportLog::SharedCanScrollToAckedLog(const FGuid& MessageId, const FConcertLogEntry& Entry)
	{
		// MessageId is shared by some logs: there is a log for receiving one one for sending for example.
		return Entry.Log.MessageId == MessageId && Entry.Log.MessageAction == EConcertLogMessageAction::Send;
	}

	void SConcertTransportLog::ScrollToLog(const int32 LogIndex) const
	{
		if (const TOptional<FPagedFilteredConcertLogList::FPageCount> PageIndex = PagedLogList->GetPageOf(LogIndex); PageIndex && PagedLogList->GetFilteredLogs().IsValidIndex(LogIndex))
		{
			PagedLogList->SetPage(*PageIndex);
			LogView->RequestNavigateToItem(PagedLogList->GetFilteredLogs()[LogIndex]);
			LogView->SetSelection(PagedLogList->GetFilteredLogs()[LogIndex]);
		}
	}
}

#undef LOCTEXT_NAMESPACE
