// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Clients/Logging/ConcertLogEntry.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::MultiUserServer { class FPagedFilteredConcertLogList; }

class FEndpointToUserNameCache;
class FMenuBuilder;
class FPagedFilteredConcertLogList;
class IConcertLogSource;
class ITableRow;
class SHeaderRow;
class SOverlay;
class SPromptConcertLoggingEnabled;
class STableViewBase;
template <typename ItemType> class SListView;

struct FColumnVisibilitySnapshot;
struct FConcertLogEntry;

namespace UE::MultiUserServer
{
	class FConcertLogFilter_FrontendRoot;
	class FConcertLogTokenizer;
	
	/**
	 * Displays the contents of a IConcertLogSource and has UI for filtering.
	 */
	class SConcertTransportLog : public SCompoundWidget
	{
	public:

		static const FName FirstColumnId;

		SLATE_BEGIN_ARGS(SConcertTransportLog)
		{}
			/** Optional filters to display in UI */
			SLATE_ARGUMENT(TSharedPtr<FConcertLogFilter_FrontendRoot>, Filter)
		SLATE_END_ARGS()
		virtual ~SConcertTransportLog() override;

		void Construct(const FArguments& InArgs, TSharedRef<IConcertLogSource> LogSource, TSharedRef<FEndpointToUserNameCache> InEndpointCache, TSharedRef<FConcertLogTokenizer> InLogTokenizer);

		bool CanScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc) const;
		void ScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc) const;
		
	private:
		
		/** Used to overlay EnableLoggingPrompt over the tabs */
		TSharedPtr<SOverlay> EnableLoggingPromptOverlay;
		/** Reminds the user to enable logging */
		TSharedPtr<SPromptConcertLoggingEnabled> EnableLoggingPrompt;

		TSharedPtr<FEndpointToUserNameCache> EndpointCache;
		
		/** Sorts the log into pages whilst applying filters */
		TSharedPtr<FPagedFilteredConcertLogList> PagedLogList;
		/** Used by various systems to convert logs to text */
		TSharedPtr<FConcertLogTokenizer> LogTokenizer;

		TSharedPtr<FConcertLogFilter_FrontendRoot> Filter;
		
		/** Lists the logs */
		TSharedPtr<SListView<TSharedPtr<FConcertLogEntry>>> LogView;
		/** Header row of LogView */
		TSharedPtr<SHeaderRow> HeaderRow;

		/** Whether to automatically scroll to new logs as they come in */
		bool bAutoScroll = true;
		/** Whether we currently loading the column visibility - prevents infinite event recursion */
		bool bIsUpdatingColumnVisibility = false;

		// Table view creation
		TSharedRef<SWidget> CreateTableView();
		TSharedRef<SHeaderRow> CreateHeaderRow();
		TSharedRef<ITableRow> OnGenerateActivityRowWidget(TSharedPtr<FConcertLogEntry> Item, const TSharedRef<STableViewBase>& OwnerTable) const;

		static void ForEachPropertyColumn(TFunctionRef<void(const FProperty& ColumnPropertyId, FName ColumnId)> Callback);

		void RestoreDefaultColumnVisiblities();
		TMap<FName, bool> GetDefaultColumnVisibilities() const;
		
		TSharedRef<SWidget> CreateViewOptionsMenu();
		void OnFilterMenuChecked();
		void OnPageViewChanged(const TArray<TSharedPtr<FConcertLogEntry>>&);
		void OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot);
		
		void OnConcertLoggingEnabledChanged(bool bNewEnabled);

		bool CanScrollToAckLog(const FGuid& MessageId) const;
		bool CanScrollToAckedLog(const FGuid& MessageId) const;
		void ScrollToAckLog(const FGuid& MessageId) const;
		void ScrollToAckedLog(const FGuid& MessageId) const;
		static bool SharedCanScrollToAckLog(const FGuid& MessageId, const FConcertLogEntry& Entry);
		static bool SharedCanScrollToAckedLog(const FGuid& MessageId, const FConcertLogEntry& Entry);
		
		void ScrollToLog(const int32 LogIndex) const;
	};
}

