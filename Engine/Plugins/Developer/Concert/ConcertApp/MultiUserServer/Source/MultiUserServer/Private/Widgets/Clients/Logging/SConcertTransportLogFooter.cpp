// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertTransportLogFooter.h"

#include "ConcertTransportEvents.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Clients/Logging/Filter/FilteredConcertLogList.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"


namespace UE::MultiUserServer
{
	namespace Private
	{
		constexpr size_t DefaultPageSizeIndex = 3; 
		static TArray<TSharedPtr<FPagedFilteredConcertLogList::FLogsPerPageCount>> PageSizeOptions = {
			MakeShared<FPagedFilteredConcertLogList::FLogsPerPageCount>(10),
			MakeShared<FPagedFilteredConcertLogList::FLogsPerPageCount>(50),
			MakeShared<FPagedFilteredConcertLogList::FLogsPerPageCount>(100),
			MakeShared<FPagedFilteredConcertLogList::FLogsPerPageCount>(500),
			MakeShared<FPagedFilteredConcertLogList::FLogsPerPageCount>(1000),
			MakeShared<FPagedFilteredConcertLogList::FLogsPerPageCount>(5000)
		};
	}

	void SConcertTransportLogFooter::Construct(const FArguments& InArgs, TSharedRef<FPagedFilteredConcertLogList> InPagedLogList)
	{
		using namespace UE::MultiUserServer::Private;
		PagedLogList = MoveTemp(InPagedLogList);

		ChildSlot
		[
			SNew(SHorizontalBox)

			// Page selection
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SSpinBox<FPagedFilteredConcertLogList::FPageCount>)
				.Value_Lambda([this](){ return PagedLogList->GetFilteredLogs().Num() == 0 ? 0 : PagedLogList->GetCurrentPage() + 1; })
				.OnValueChanged_Lambda([this](FPagedFilteredConcertLogList::FPageCount NewValue) { PagedLogList->SetPage(NewValue - 1); })
				.MinValue(0)
				.MaxValue_Lambda([this]() { return PagedLogList->GetNumPages(); })
			]

			// Page counter
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Lambda([this](){ return FText::Format(LOCTEXT("PageCounterFormat", "of {0} {0}|plural(one=page,other=pages)"), PagedLogList->GetNumPages()); })
			]

			// Logs per page text
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(30, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LogePerPageText", "Logs per page"))
			]
			
			// Logs per page combo box
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SComboBox<TSharedPtr<FPagedFilteredConcertLogList::FLogsPerPageCount>>)
				.InitiallySelectedItem(PageSizeOptions[DefaultPageSizeIndex])
				.OnSelectionChanged_Lambda([this](const TSharedPtr<FPagedFilteredConcertLogList::FLogsPerPageCount>& NewCount, ESelectInfo::Type){ PagedLogList->SetLogsPerPage(*NewCount); })
				.OnGenerateWidget_Lambda([](const TSharedPtr<FPagedFilteredConcertLogList::FLogsPerPageCount>& Count) { return SNew(STextBlock).Text(FText::FromString(FString::FromInt(*Count))); })
				.OptionsSource(&PageSizeOptions)
				[
					SNew(STextBlock).Text_Lambda([this](){ return FText::FromString(FString::FromInt(PagedLogList->GetLogsPerPage())); })
				]
			]

			// Log counter
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(30, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return PagedLogList->IsBufferFull()
						? FText::Format(LOCTEXT("LogCounterFormat.Full", "{0} filtered {0}|plural(one=log,other=logs) (buffer full)"), PagedLogList->GetFilteredLogs().Num())
						: FText::Format(LOCTEXT("LogCounterFormat.NotFull", "{0} filtered {0}|plural(one=log,other=logs)"), PagedLogList->GetFilteredLogs().Num());
				})
			]

			// Gap Filler
			+SHorizontalBox::Slot()
			.FillWidth(1.0)
			[
				SNew(SSpacer)
			]
			// Disable logging button
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked_Lambda([](){ ConcertTransportEvents::SetLoggingEnabled(false); return FReply::Handled(); })
				.ToolTipText(LOCTEXT("DisableLoggingVisibility.Button.Tooltip", "Executes Concert.DisableLogging"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DisableLoggingVisibility.Button.Text", "Disable Logging"))
				]
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE
