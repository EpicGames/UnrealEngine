// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filter/FilteredConcertLogList.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class STextBlock;
template<typename T> class SSpinBox;
struct FConcertLogEntry;

namespace UE::MultiUserServer
{
	/** Displays the number of pages and items. Displayed under the table view. */
	class SConcertTransportLogFooter : public SCompoundWidget
	{
	public:
	
		DECLARE_DELEGATE_OneParam(FExtendContextMenu, FMenuBuilder&)
			SLATE_BEGIN_ARGS(SConcertTransportLogFooter) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FPagedFilteredConcertLogList> InPagedLogList);

	private:

		/** The model we'll be updating */
		TSharedPtr<FPagedFilteredConcertLogList> PagedLogList;

		/** Selects the current page */
		TSharedPtr<SSpinBox<FPagedFilteredConcertLogList::FPageCount>> CurrentPage;
		/** Displays the number of pages */
		TSharedPtr<STextBlock> PageCounterText;
	};
}
