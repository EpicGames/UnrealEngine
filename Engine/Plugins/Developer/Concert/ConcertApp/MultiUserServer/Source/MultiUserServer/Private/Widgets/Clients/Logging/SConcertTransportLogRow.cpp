// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertTransportLogRow.h"

#include "ConcertServerStyle.h"
#include "SConcertTransportLog.h"
#include "Util/ConcertLogTokenizer.h"

#include "Algo/AnyOf.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertTransportLogRow"

namespace UE::MultiUserServer
{
	void SConcertTransportLogRow::Construct(const FArguments& InArgs, TSharedPtr<FConcertLogEntry> InLogEntry, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<FConcertLogTokenizer> InTokenizer)
	{
		LogEntry = MoveTemp(InLogEntry);
		Tokenizer = MoveTemp(InTokenizer);
		HighlightText = InArgs._HighlightText;
		
		AvatarColor = InArgs._AvatarColor;
		CanScrollToAckLogFunc = InArgs._CanScrollToAckLog;
		CanScrollToAckedLogFunc = InArgs._CanScrollToAckedLog;
		ScrollToAckLogFunc = InArgs._ScrollToAckLog;
		ScrollToAckedLogFunc = InArgs._ScrollToAckedLog;
		check(ScrollToAckLogFunc.IsBound() && ScrollToAckedLogFunc.IsBound());
		
		SMultiColumnTableRow<TSharedPtr<FConcertLogEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	TSharedRef<SWidget> SConcertTransportLogRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		if (ColumnName == SConcertTransportLog::FirstColumnId)
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(2, 1)
				[
					SNew(SColorBlock)
					.Color(AvatarColor)
					.Size(FVector2D(4.f, 16.f))
				];
		}
		
		using FColumnWidgetFactoryFunc = TSharedRef<SWidget>(SConcertTransportLogRow::*)(const FName& InColumnName);
		const TMap<FName, FColumnWidgetFactoryFunc> OverrideFactories = {
			{ GET_MEMBER_NAME_CHECKED(FConcertLogMetadata, AckState), &SConcertTransportLogRow::CreateAckColumn }
		};
		const FColumnWidgetFactoryFunc FallbackFactoryFunc = &SConcertTransportLogRow::CreateDefaultColumn;

		const FColumnWidgetFactoryFunc* FactoryFunc = OverrideFactories.Find(ColumnName);
		return FactoryFunc
			? Invoke(*FactoryFunc, this, ColumnName)
			: Invoke(FallbackFactoryFunc, this, ColumnName);
	}

	TSharedRef<SWidget> SConcertTransportLogRow::CreateDefaultColumn(const FName& PropertyName)
	{
		return SNew(STextBlock)
			.Text_Lambda([PropertyName, WeakEntry = TWeakPtr<FConcertLogEntry>(LogEntry), WeakTokenizer = TWeakPtr<FConcertLogTokenizer>(Tokenizer)]()
			{
				TSharedPtr<FConcertLogEntry> PinnedEntry = WeakEntry.Pin();
				TSharedPtr<FConcertLogTokenizer> PinnedTokenizer = WeakTokenizer.Pin();
				if (!ensure(PinnedEntry && PinnedTokenizer))
				{
					return FText::GetEmpty();
				}
				
				const FProperty* ConcertLogProperty = FConcertLog::StaticStruct()->FindPropertyByName(PropertyName);
				if (ConcertLogProperty)
				{
					return FText::FromString(PinnedTokenizer->Tokenize(PinnedEntry->Log, *ConcertLogProperty));
				}

				const FProperty* ConcertLogEntryProperty = FConcertLogMetadata::StaticStruct()->FindPropertyByName(PropertyName);
				if (ConcertLogEntryProperty)
				{
					return FText::FromString(PinnedTokenizer->Tokenize(PinnedEntry->LogMetaData, *ConcertLogEntryProperty));
				}
					
				return FText::GetEmpty();
			})
			.HighlightText(HighlightText);
	}

	TSharedRef<SWidget> SConcertTransportLogRow::CreateAckColumn(const FName& PropertyName)
	{
		return SNew(SOverlay)

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateAckInProgressWidget()
			]
		
			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateAckFailureWidget()
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateAckNotNeededWidget()
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateAckSuccessWidget()
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateAckWidget()
			]
		;
	}

	TSharedRef<SWidget> SConcertTransportLogRow::CreateAckInProgressWidget() const
	{
		return SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SThrobber)
				.Animate(SThrobber::VerticalAndOpacity)
				.Visibility_Lambda([this]()
				{
					return LogEntry->LogMetaData.AckState == EConcertLogAckState::InProgress
						? EVisibility::Visible
						: EVisibility::Hidden;
				})
				.ToolTipText(LOCTEXT("InProgress", "Awaiting ACK..."))
			];
	}

	TSharedRef<SWidget> SConcertTransportLogRow::CreateAckSuccessWidget() const
	{
		return SNew(SButton)
			.ContentPadding(0.f)
			.IsEnabled_Lambda([this]()
			{
				return LogEntry->LogMetaData.AckingMessageId.IsSet() && CanScrollToAckLogFunc.Execute(*LogEntry->LogMetaData.AckingMessageId);
			})
			.OnPressed_Lambda([this]()
			{
				if (LogEntry->LogMetaData.AckingMessageId.IsSet())
				{
					ScrollToAckLogFunc.Execute(*LogEntry->LogMetaData.AckingMessageId);
				}
			})
			.Visibility_Lambda([this]()
			{
				return LogEntry->LogMetaData.AckState == EConcertLogAckState::AckReceived
					? EVisibility::Visible
					: EVisibility::Hidden;
			})
			.ToolTipText_Lambda([this]()
			{
				return LogEntry->LogMetaData.AckingMessageId.IsSet() && CanScrollToAckLogFunc.Execute(*LogEntry->LogMetaData.AckingMessageId)
					? LOCTEXT("Acked.Scroll.Enabled", "ACK received. Press to jump to the log that ACKs this log")
					: LOCTEXT("Acked.Scroll.Disabled", "ACK received\nThis button is disabled because the corresponding ACK log is filtered out.");
			})
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
				.Image(FConcertServerStyle::Get().GetBrush("Concert.Ack.Success"))
			];
	}

	TSharedRef<SWidget> SConcertTransportLogRow::CreateAckFailureWidget() const
	{
		return SNew(SImage)
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
			.Image(FConcertServerStyle::Get().GetBrush("Concert.Ack.Failure"))
			.Visibility_Lambda([this]()
			{
				return LogEntry->LogMetaData.AckState == EConcertLogAckState::AckFailure
					? EVisibility::Visible
					: EVisibility::Hidden;
			})
			.ToolTipText(LOCTEXT("AckFailure", "No ACK received: retries exhausted."));
	}

	TSharedRef<SWidget> SConcertTransportLogRow::CreateAckNotNeededWidget() const
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NotApplicable", "n/a"))
			.Visibility_Lambda([this]()
			{
				return LogEntry->LogMetaData.AckState == EConcertLogAckState::NotNeeded
					? EVisibility::Visible
					: EVisibility::Hidden;
			})
			.ToolTipText(LOCTEXT("Received", "No ACK needed."));
	}

	TSharedRef<SWidget> SConcertTransportLogRow::CreateAckWidget() const
	{
		return SNew(SComboButton)
			.HasDownArrow(false)
			.ToolTipText_Lambda([this]()
			{
				const bool bIsEnabled = LogEntry->LogMetaData.AckedMessageId && Algo::AnyOf(*LogEntry->LogMetaData.AckedMessageId, [this](const FGuid& MessageId)
					{
					return CanScrollToAckedLogFunc.Execute(MessageId);
				});
				return bIsEnabled
					? LOCTEXT("Ack.Scroll.Disabled", "This is an ACK. Press to jump to the log this log ACKs.")
					: LOCTEXT("Ack.Scroll.Enabled", "This is an ACK.\nThis button is disabled because all logs this log ACKs are filtered out.");
			})
			.IsEnabled_Lambda([this]()
			{
				return LogEntry->LogMetaData.AckedMessageId && Algo::AnyOf(*LogEntry->LogMetaData.AckedMessageId, [this](const FGuid& MessageId)
				{
					return CanScrollToAckedLogFunc.Execute(MessageId);
				});
			})
			.OnGetMenuContent_Lambda([this]()
			{
				FMenuBuilder MenuBuilder(true, nullptr);
				for (const FGuid& MessageId : *LogEntry->LogMetaData.AckedMessageId)
				{
					MenuBuilder.AddMenuEntry(
						FText::FromString(MessageId.ToString(EGuidFormats::DigitsWithHyphens)),
						CanScrollToAckedLogFunc.Execute(MessageId) ? FText() : LOCTEXT("Ack.Scroll.ComoboEntry.Disabled", "This log is filtered out."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, MessageId]()
							{
								ScrollToAckedLogFunc.Execute(MessageId);
							}),
							FCanExecuteAction::CreateLambda([this, MessageId] { return CanScrollToAckedLogFunc.Execute(MessageId); })),
						NAME_None,
						EUserInterfaceActionType::Button
					);
				}
				return MenuBuilder.MakeWidget();
			})
			.Visibility_Lambda([this]()
			{
				return LogEntry->LogMetaData.AckState == EConcertLogAckState::Ack
					? EVisibility::Visible
					: EVisibility::Hidden;
			})
			.ButtonContent()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
				.Image(FConcertServerStyle::Get().GetBrush("Concert.Ack.Ack"))
			];
	}
}

#undef LOCTEXT_NAMESPACE
