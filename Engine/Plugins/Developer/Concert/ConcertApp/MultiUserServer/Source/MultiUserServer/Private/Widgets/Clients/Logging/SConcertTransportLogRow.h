// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;
struct FConcertLogEntry;


namespace UE::MultiUserServer
{
	class FConcertLogTokenizer;
	
	class SConcertTransportLogRow : public SMultiColumnTableRow<TSharedPtr<FConcertLogEntry>>
	{
	public:

		DECLARE_DELEGATE_OneParam(FScrollToLog, const FGuid& /*MessageId*/);
		DECLARE_DELEGATE_RetVal_OneParam(bool, FCanScrollToLog, const FGuid& /*MessageId*/);

		SLATE_BEGIN_ARGS(SConcertTransportLogRow)
		{}
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(FLinearColor, AvatarColor)
		SLATE_EVENT(FCanScrollToLog, CanScrollToAckLog)
		SLATE_EVENT(FCanScrollToLog, CanScrollToAckedLog)
		SLATE_EVENT(FScrollToLog, ScrollToAckLog)
		SLATE_EVENT(FScrollToLog, ScrollToAckedLog)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FConcertLogEntry> InLogEntry, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<FConcertLogTokenizer> InTokenizer);

		//~ Begin SMultiColumnTableRow Interface
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
		//~ End SMultiColumnTableRow Interface
	
		private:

		TSharedPtr<FConcertLogEntry> LogEntry;
		/** Used to convert some members into display strings */
		TSharedPtr<FConcertLogTokenizer> Tokenizer;

		TAttribute<FText> HighlightText;

		FLinearColor AvatarColor;
		FCanScrollToLog CanScrollToAckLogFunc;
		FCanScrollToLog CanScrollToAckedLogFunc;
		FScrollToLog ScrollToAckLogFunc;
		FScrollToLog ScrollToAckedLogFunc;

		TSharedRef<SWidget> CreateDefaultColumn(const FName& PropertyName);
	
		TSharedRef<SWidget> CreateAckColumn(const FName& PropertyName);
		TSharedRef<SWidget> CreateAckInProgressWidget() const;
		TSharedRef<SWidget> CreateAckSuccessWidget() const;
		TSharedRef<SWidget> CreateAckFailureWidget() const;
		TSharedRef<SWidget> CreateAckNotNeededWidget() const;
		TSharedRef<SWidget> CreateAckWidget() const;
	
	};
}
