// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalLogSource.h"



namespace UE::MultiUserServer::Private
{
	static constexpr FConcertLogID InvalidID = TNumericLimits<FConcertLogID>::Max();
}

FGlobalLogSource::FGlobalLogSource(size_t LogCapacity)
	: LogPtrs(TCircularBuffer<TSharedPtr<FConcertLogEntry>>(LogCapacity))
	, LogEntries(TCircularBuffer<FConcertLogEntry>(LogCapacity))
{
	checkf(LogCapacity > 2, TEXT("Minimum log capacity must be 2 for AddLog's invariant for the lowest new ID to hold"));

	for (size_t i = 0; i < LogPtrs.Capacity(); ++i)
	{
		LogPtrs[i] = TSharedPtr<FConcertLogEntry>(
			// Our buffer holds the memory
			&LogEntries[i],
			// No additional work is needed to "free" the memory
			[](auto){}
			);
		
		LogEntries[i].LogId = UE::MultiUserServer::Private::InvalidID;
	}

	ConcertTransportEvents::OnConcertServerLogEvent().AddRaw(this, &FGlobalLogSource::OnLogProduced);
}

FGlobalLogSource::~FGlobalLogSource()
{
	ConcertTransportEvents::OnConcertServerLogEvent().RemoveAll(this);
}

void FGlobalLogSource::OnLogProduced(const IConcertServer&, const FConcertLog& Log)
{
	check(IsInGameThread());
	AddLog(Log);
}

void FGlobalLogSource::AddLog(const FConcertLog& NewLog)
{
	const TSharedRef<FConcertLogEntry> NewEntry = LogPtrs[NextIndexToAddTo].ToSharedRef();

	const bool bNeedToOverrideOldEntry = NewEntry->LogId != UE::MultiUserServer::Private::InvalidID; 
	if (bNeedToOverrideOldEntry)
	{
		// We've wrapped around all elements: we've just reached the smallest because IDs grow strongly monotonously.
		// Since there are at least two entries (see constructor) and because of the above, the next element has new lowest ID.
		OldestLogEntryIndex = NextIndexToAddTo;
		OnLowestLogEntryChanged().Broadcast(NewEntry->LogId + 1);
		// Everybody that had been the log pointer now knows it is pointing to a new log
	}

	NewEntry->LogId = NextLogID;
	NewEntry->Log = NewLog;
	
	NextIndexToAddTo = LogPtrs.GetNextIndex(NextIndexToAddTo);
	++NextLogID;
	NumberOfLogs = FMath::Min<size_t>(NumberOfLogs + 1, LogPtrs.Capacity());
	
	// Could happen if we left it running for 550000 years at 1 million logs per second
	checkf(NextLogID != UE::MultiUserServer::Private::InvalidID, TEXT("Reached max possible log ID"));
	
	OnLogEntryAdded().Broadcast(NewEntry);
}
