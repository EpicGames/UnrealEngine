// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertLogSource.h"
#include "Containers/CircularBuffer.h"

struct FConcertLog;

class IConcertServer;

/**
 * Listens for log events using ConcertTransportEvents::OnConcertServerLogEvent and buffers them.
 *
 * The shared pointers and log entries are allocated in two sequential buffers to improve memory footprint and cache
 * coherency; the footprint is improved because two big chunks of memory is better than many tiny chunks.
 */
class FGlobalLogSource : public IConcertLogSource
{
public:
	
	FGlobalLogSource(size_t LogCapacity);
	virtual ~FGlobalLogSource() override;

	//~ Begin IConcertLogSource Interface
	virtual bool IsBufferFull() const override { return NumberOfLogs == LogPtrs.Capacity(); }
	virtual size_t GetNumLogs() const override { return NumberOfLogs; }
	virtual const TSharedPtr<FConcertLogEntry>& GetElement_Unchecked(size_t Index) const override { return LogPtrs[OldestLogEntryIndex + Index]; }
	virtual FLogEntryIdEvent& OnLowestLogEntryChanged() override { return OnLowestEntryChangedEvent; }
	virtual FLogEntryEvent& OnLogEntryAdded() override { return OnLogEntryAddedEvent; }
	//~ End IConcertLogSource Interface

private:

	/** The lowest ID in the buffer */
	size_t OldestLogEntryIndex = 0;
	/** The number of logs in the buffer. */
	size_t NumberOfLogs = 0;
	/** The next index at which to insert a log entry */
	size_t NextIndexToAddTo = 0;
	/** The next log ID to use */
	FConcertLogID NextLogID = 0;
	
	/** Contains all logs. The oldest logs are discarded */
	TCircularBuffer<TSharedPtr<FConcertLogEntry>> LogPtrs;
	/** The content that the LogPtrs point to. For better cache coherency. */
	TCircularBuffer<FConcertLogEntry> LogEntries;
	// TODO: It would be great to add another TCircularBuffer for the pointers' SharedReferenceCount for a better memory footprint

	FLogEntryIdEvent OnLowestEntryChangedEvent;
	FLogEntryEvent OnLogEntryAddedEvent;

	void OnLogProduced(const IConcertServer&, const FConcertLog& Log);
	void AddLog(const FConcertLog& NewLog);
};
