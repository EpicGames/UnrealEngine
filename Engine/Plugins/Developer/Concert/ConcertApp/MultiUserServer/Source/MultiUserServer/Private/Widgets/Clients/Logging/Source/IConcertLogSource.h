// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Clients/Logging/ConcertLogEntry.h"

/**
 * Describes a source of logs - it is similar to an array but with some changes:
 * - GetElement_Unchecked is used to access elements by index, as usual.
 * - the highest valid index is NumLogs() - 1, as usual.
 * - the lowest valid index is not necessarily 0 but is instead LowestLogIndex().
 *
 * Implementations can decide to delete the oldest logs if the number of logs grows too high; the implementation will call
 * OnLowestLogEntryChanged when old logs are removed.
 */
class IConcertLogSource
{
public:
	
	virtual ~IConcertLogSource() = default;

	/** @return Whether the maximum number of logs has been reached */
	virtual bool IsBufferFull() const = 0;
	
	/** @return The number of logs in this source */
	virtual size_t GetNumLogs() const = 0;
	/**
	 * @param Index Satisfies 0 <= Index < NumLogs(). Index 0 is the oldest log.
	 * @return The log entry at the given Index - crashes if the index is out of bounds.
	 */
	virtual const TSharedPtr<FConcertLogEntry>& GetElement_Unchecked(size_t Index) const = 0;

	void ForEachLog(TFunctionRef<void(const TSharedPtr<FConcertLogEntry>& Log)> ConsumerFunc) const
	{
		for (size_t i = 0; i < GetNumLogs(); ++i)
		{
			ConsumerFunc(GetElement_Unchecked(i));
		}
	}

	DECLARE_MULTICAST_DELEGATE_OneParam(FLogEntryIdEvent, FConcertLogID /*LogID*/);
	/**
	 * Called when the log buffer is about to remove the oldest logs.
	 * The event tells the lowest ID that is still valid.
	 * 
	 * Once this event has completed, the log data is overwritten with the latest log data.
	 */
	virtual FLogEntryIdEvent& OnLowestLogEntryChanged() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FLogEntryEvent, const TSharedRef<FConcertLogEntry>& /*NewEntry*/);
	/** Called when a new entry has been added to the log buffer. */
	virtual FLogEntryEvent& OnLogEntryAdded() = 0;
};
