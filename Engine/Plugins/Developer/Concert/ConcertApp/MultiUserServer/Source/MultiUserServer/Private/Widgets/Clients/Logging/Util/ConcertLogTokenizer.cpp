// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogTokenizer.h"

#include "ConcertTransportEvents.h"
#include "MessageTypeUtils.h"
#include "Math/UnitConversion.h"
#include "Settings/ConcertTransportLogSettings.h"
#include "Widgets/Clients/Util/EndpointToUserNameCache.h"

namespace UE::MultiUserServer
{
	FConcertLogTokenizer::FConcertLogTokenizer(TSharedRef<FEndpointToUserNameCache> EndpointInfoGetter)
		: EndpointInfoGetter(EndpointInfoGetter)
	{
		TokenizerFunctions = {
				{ FConcertLog::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FConcertLog, MessageId)), [this](const FConcertLog& Log) { return TokenizeMessageId(Log); } },
				{ FConcertLog::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FConcertLog, Timestamp)), [this](const FConcertLog& Log) { return TokenizeTimestamp(Log); } },
				{ FConcertLog::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FConcertLog, MessageTypeName)), [this](const FConcertLog& Log) { return TokenizeMessageTypeName(Log); } },
				{ FConcertLog::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FConcertLog, CustomPayloadUncompressedByteSize)), [this](const FConcertLog& Log) { return TokenizeCustomPayloadUncompressedByteSize(Log); } },
				{ FConcertLog::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FConcertLog, OriginEndpointId)), [this](const FConcertLog& Log) { return TokenizeOriginEndpointId(Log); } },
				{ FConcertLog::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FConcertLog, DestinationEndpointId)), [this](const FConcertLog& Log) { return TokenizeDestinationEndpointId(Log); } }
		};
	}

	FString FConcertLogTokenizer::Tokenize(const FConcertLog& Data, const FProperty& ConcertLogProperty) const
	{
		if (const FTokenizeFunc* CustomTokenizer = TokenizerFunctions.Find(&ConcertLogProperty))
		{
			return (*CustomTokenizer)(Data);
		}
		return TokenizeUsingPropertyExport(&Data, ConcertLogProperty);
	}

	FString FConcertLogTokenizer::Tokenize(const FConcertLogMetadata& Data, const FProperty& ConcertLogMetadataProperty) const
	{
		return TokenizeUsingPropertyExport(&Data, ConcertLogMetadataProperty);
	}

	FString FConcertLogTokenizer::TokenizeMessageId(const FConcertLog& Data) const
	{
		return Data.MessageId.ToString(EGuidFormats::DigitsWithHyphens);
	}

	FString FConcertLogTokenizer::TokenizeTimestamp(const FConcertLog& Data) const
	{
		return ConcertFrontendUtils::FormatTime(Data.Timestamp, UConcertTransportLogSettings::GetSettings()->TimestampTimeFormat).ToString();
	}

	FString FConcertLogTokenizer::TokenizeMessageTypeName(const FConcertLog& Data) const
	{
		return UE::MultiUserServer::MessageTypeUtils::SanitizeMessageTypeName(Data.MessageTypeName);
	}

	FString FConcertLogTokenizer::TokenizeCustomPayloadUncompressedByteSize(const FConcertLog& Data) const
	{
		// So changes to the type are automatically propagated here
		const FNumericUnit<int32> DisplayUnit = FUnitConversion::QuantizeUnitsToBestFit(Data.CustomPayloadUncompressedByteSize, EUnit::Bytes);
		return FString::Printf(TEXT("%d %s"), DisplayUnit.Value, FUnitConversion::GetUnitDisplayString(DisplayUnit.Units));
	}

	FString FConcertLogTokenizer::TokenizeOriginEndpointId(const FConcertLog& Data) const
	{
		return EndpointInfoGetter->GetEndpointDisplayString(Data.OriginEndpointId);
	}

	FString FConcertLogTokenizer::TokenizeDestinationEndpointId(const FConcertLog& Data) const
	{
		return EndpointInfoGetter->GetEndpointDisplayString(Data.DestinationEndpointId);
	}

	FString FConcertLogTokenizer::TokenizeUsingPropertyExport(const void* ContainerPtr, const FProperty& ConcertLogProperty) const
	{
		FString Exported;
		const void* ValuePtr = ConcertLogProperty.ContainerPtrToValuePtr<void>(ContainerPtr);
		const void* DeltaPtr = ValuePtr; // We have no real delta - in this case the API expects the same ptr
		const bool bSuccess = ConcertLogProperty.ExportText_Direct(Exported, ValuePtr, DeltaPtr, nullptr, PPF_ExternalEditor);
		check(bSuccess);
		return Exported;
	}
}
