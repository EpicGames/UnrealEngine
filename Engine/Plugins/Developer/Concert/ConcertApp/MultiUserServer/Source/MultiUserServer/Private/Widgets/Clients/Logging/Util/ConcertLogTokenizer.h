// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FEndpointToUserNameCache;
struct FConcertLog;
struct FConcertLogMetadata;

namespace UE::MultiUserServer
{
	/** Converts members of FConcertLog and FConcertLogMetadata into a string. Used e.g. to make search respect the display settings. */
	class FConcertLogTokenizer
	{
	public:

		FConcertLogTokenizer(TSharedRef<FEndpointToUserNameCache> EndpointInfoGetter);

		/** Tokenizes a property of FConcertLog into a string */
		FString Tokenize(const FConcertLog& Data, const FProperty& ConcertLogProperty) const;
		/** Tokenizes a property of FConcertLogMetadata into a string */
		FString Tokenize(const FConcertLogMetadata& Data, const FProperty& ConcertLogMetadataProperty) const;

		// ContainerPtr Tokenizers
		FString TokenizeMessageId(const FConcertLog& Data) const;
		FString TokenizeTimestamp(const FConcertLog& Data) const;
		FString TokenizeMessageTypeName(const FConcertLog& Data) const;
		FString TokenizeCustomPayloadUncompressedByteSize(const FConcertLog& Data) const;
		FString TokenizeOriginEndpointId(const FConcertLog& Data) const;
		FString TokenizeDestinationEndpointId(const FConcertLog& Data) const;

	private:

		using FTokenizeFunc = TFunction<FString(const FConcertLog&)>;

		/** Override functions for tokenizing certain properties */
		TMap<const FProperty*, FTokenizeFunc> TokenizerFunctions;

		/** Used so we can look up client and server info (even after client has disconnected) */
		TSharedRef<FEndpointToUserNameCache> EndpointInfoGetter;
	
		FString TokenizeUsingPropertyExport(const void* ContainerPtr, const FProperty& ConcertLogProperty) const;
	};
}

