// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogAckTracker.h"

#include "IConcertServer.h"

#include "Widgets/Clients/Logging/Source/IConcertLogSource.h"

FLogAckTracker::FLogAckTracker(TSharedRef<IConcertLogSource> InLogSource, TSharedRef<IConcertServer> InServer)
	: LogSource(MoveTemp(InLogSource))
	, Server(MoveTemp(InServer))
{
	LogSource->OnLogEntryAdded().AddRaw(this, &FLogAckTracker::OnLogEntryProduced);
	Server->OnConcertMessageAcknowledgementReceived().AddRaw(this, &FLogAckTracker::OnAckProcessed);
}

FLogAckTracker::~FLogAckTracker()
{
	LogSource->OnLogEntryAdded().RemoveAll(this);
	Server->OnConcertMessageAcknowledgementReceived().RemoveAll(this);
}

void FLogAckTracker::OnLogEntryProduced(const TSharedRef<FConcertLogEntry>& LogEntry)
{
	const bool bIsAck = LogEntry->Log.MessageTypeName == FConcertAckData::StaticStruct()->GetFName();
	if (bIsAck)
	{
		if (LogEntry->Log.MessageAction == EConcertLogMessageAction::Process)
		{
			LogEntry->LogMetaData.AckState = EConcertLogAckState::Ack;
			TSet<FGuid> AckedMessageIds;
			AckingMessageToAckedMessages.RemoveAndCopyValue(LogEntry->Log.MessageId, AckedMessageIds);
			LogEntry->LogMetaData.AckedMessageId = AckedMessageIds;
		}
		return;
	}
	
	if (LogEntry->Log.MessageAction == EConcertLogMessageAction::TimeOut)
	{
		if (const FPendingAckInfo* TimedOutPending = PendingAckLogs.Find(LogEntry->Log.MessageId); TimedOutPending )
		{
			for (const TWeakPtr<FConcertLogEntry>& TimedOut : TimedOutPending->LogEntries)
			{
				TimedOut.Pin()->LogMetaData.AckState = EConcertLogAckState::AckFailure;
			}
		}
		PendingAckLogs.Remove(LogEntry->Log.MessageId);
		return;
	}

	const TSet<EConcertLogMessageAction> AckNeeded = { EConcertLogMessageAction::Send };
	const bool bIsReliable = LogEntry->Log.ChannelId != FConcertMessageData::UnreliableChannelId;
	if (bIsReliable && AckNeeded.Contains(LogEntry->Log.MessageAction))
	{
		LogEntry->LogMetaData.AckState = EConcertLogAckState::InProgress;
		PendingAckLogs.FindOrAdd(LogEntry->Log.MessageId).LogEntries.Add(LogEntry);
	}
	else
	{
		LogEntry->LogMetaData.AckState = EConcertLogAckState::NotNeeded;
	}
}

void FLogAckTracker::OnAckProcessed(
	const FConcertEndpointContext& LocalEndpointId,
	const FConcertEndpointContext& RemoteEndpointId,
	const TSharedRef<IConcertMessage>& AckedMessage,
	const FConcertMessageContext& MessageContext)
{
	if (const FPendingAckInfo* Pending = PendingAckLogs.Find(AckedMessage->GetMessageId()); Pending)
	{
		AckingMessageToAckedMessages.FindOrAdd(MessageContext.Message->MessageId).Add(AckedMessage->GetMessageId());
		for (const TWeakPtr<FConcertLogEntry>& LogEntry : Pending->LogEntries)
		{
			TSharedPtr<FConcertLogEntry> Pinned = LogEntry.Pin();
			Pinned->LogMetaData.AckState = EConcertLogAckState::AckReceived;
			Pinned->LogMetaData.AckingMessageId = MessageContext.Message->MessageId;
		}
		
		PendingAckLogs.Remove(AckedMessage->GetMessageId());
	}
}
