// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

class IConcertMessage;
struct FConcertEndpointContext;
struct FConcertMessageContext;

class FStructOnScope;
class IConcertLogSource;
class IConcertServer;

struct FConcertLogEntry;

/** Tracks ack logs and updates the  */
class FLogAckTracker
{
public:
	FLogAckTracker(TSharedRef<IConcertLogSource> LogSource, TSharedRef<IConcertServer> Server);
	~FLogAckTracker();

private:

	/** We analyse & track newly added logs */
	const TSharedRef<IConcertLogSource> LogSource;
	/** We listen to received ACKs from the server. */
	TSharedRef<IConcertServer> Server;

	struct FPendingAckInfo
	{
		/** There can be several logs with the same Message ID */
		TArray<TWeakPtr<FConcertLogEntry>> LogEntries;
	};
	
	/** Logs we're waiting on an ACK for */
	TMap<FGuid, FPendingAckInfo> PendingAckLogs;

	/** Maps an ACK log to the message IDs it ACKs. */
	TMap<FGuid, TSet<FGuid>> AckingMessageToAckedMessages;
	
	void OnLogEntryProduced(const TSharedRef<FConcertLogEntry>& LogEntry);
	void OnAckProcessed(const FConcertEndpointContext& LocalEndpointId, const FConcertEndpointContext& RemoteEndpointId, const TSharedRef<IConcertMessage>& AckedMessage, const FConcertMessageContext& MessageContext);
};
