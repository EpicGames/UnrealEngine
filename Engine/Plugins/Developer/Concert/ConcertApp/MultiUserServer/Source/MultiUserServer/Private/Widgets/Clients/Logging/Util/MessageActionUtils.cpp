// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageActionUtils.h"

#include "ConcertTransportEvents.h"

namespace UE::MultiUserServer::MessageActionUtils
{
	TSet<FName> GetAllMessageActionNames()
	{
		static const TSet<FName> CachedResult = []()
		{
			const TSet<EConcertLogMessageAction> Actions = GetAllMessageActions();
			TSet<FName> Result;
			Algo::Transform(Actions, Result, [](EConcertLogMessageAction Action)
			{
				return ConvertActionToName(Action);
			});
			return Result;
		}();
		return CachedResult;
	}

	TSet<EConcertLogMessageAction> GetAllMessageActions()
	{
		constexpr int32 NumActions = 12;
		static_assert(static_cast<int32>(EConcertLogMessageAction::EndpointClosure) == NumActions - 1, "Update this function when you change EConcertLogMessageAction");
		static_assert(static_cast<int32>(EConcertLogMessageAction::None) == 0, "Fix the below loop when you change EConcertLogMessageAction");

		TSet<EConcertLogMessageAction> Result;
		for (int32 i = 0; i < NumActions; ++i)
		{
			Result.Add(static_cast<EConcertLogMessageAction>(i));
		}

		return Result;
	}

	FName ConvertActionToName(EConcertLogMessageAction MessageAction)
	{
		return UEnum::GetValueAsName(MessageAction);
	}

	FString GetActionDisplayString(EConcertLogMessageAction MessageAction)
	{
		const FName Name = ConvertActionToName(MessageAction);
		return GetActionDisplayString(Name);
	}

	FString GetActionDisplayString(FName MessageActionName)
	{
		FString AsString = MessageActionName.ToString();
		AsString.RemoveFromStart(TEXT("EConcertLogMessageAction::"));
		return AsString;
	}
}
