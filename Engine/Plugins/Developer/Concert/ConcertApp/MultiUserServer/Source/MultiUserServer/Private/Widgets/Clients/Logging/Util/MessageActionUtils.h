// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"

enum class EConcertLogMessageAction : uint8;

namespace UE::MultiUserServer::MessageActionUtils
{
	/** Gets all message action names */
	TSet<FName> GetAllMessageActionNames();
	TSet<EConcertLogMessageAction> GetAllMessageActions();

	FName ConvertActionToName(EConcertLogMessageAction MessageAction);
	FString GetActionDisplayString(EConcertLogMessageAction MessageAction);
	FString GetActionDisplayString(FName MessageActionName);
};
