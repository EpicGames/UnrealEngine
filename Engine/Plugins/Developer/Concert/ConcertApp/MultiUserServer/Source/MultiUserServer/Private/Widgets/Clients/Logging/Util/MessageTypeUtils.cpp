// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageTypeUtils.h"

#include "ConcertTransportMessages.h"
#include "UObject/UObjectIterator.h"

namespace UE::MultiUserServer::MessageTypeUtils
{
	static TSet<FName> FilterAllMessageTypes(TFunctionRef<bool(UStruct*)> ConsumerFunc);
	
	TSet<FName> GetAllMessageTypeNames()
	{
		static TSet<FName> CachedSubStructs = FilterAllMessageTypes([](UStruct* MessageType)
		{
			const TSet<UStruct*> Excluded = {
				FConcertMessageData::StaticStruct(),
				FConcertEventData::StaticStruct(),
				FConcertRequestData::StaticStruct(),
				FConcertResponseData::StaticStruct()
			};

			return !Excluded.Contains(MessageType);
		});

		return CachedSubStructs;
	}

	TSet<FName> GetAllMessageTypeNames_EventsOnly()
	{
		static TSet<FName> CachedSubStructs = FilterAllMessageTypes([](UStruct* MessageType)
		{
			return MessageType->IsChildOf(FConcertEventData::StaticStruct());
		});

		return CachedSubStructs;
	}
	
	TSet<FName> GetAllMessageTypeNames_RequestsOnly()
	{
		static TSet<FName> CachedSubStructs = FilterAllMessageTypes([](UStruct* MessageType)
		{
			return MessageType->IsChildOf(FConcertRequestData::StaticStruct());
		});

		return CachedSubStructs;
	}
	
	TSet<FName> GetAllMessageTypeNames_ResponseOnlyOnly()
	{
		static TSet<FName> CachedSubStructs = FilterAllMessageTypes([](UStruct* MessageType)
		{
			return MessageType->IsChildOf(FConcertResponseData::StaticStruct());
		});

		return CachedSubStructs;
	}
	
	TSet<FName> GetAllMessageTypeNames_AcksOnlyOnly()
	{
		return { FConcertAckData::StaticStruct()->GetFName() };
	}

	FString SanitizeMessageTypeName(FName MessageTypeName)
	{
		static TMap<FName, FString> CachedSanitizedNames = []()
		{
			TMap<FName, FString> Result;
			const TSet<UStruct*> Excluded = {
				FConcertMessageData::StaticStruct(),
				FConcertEventData::StaticStruct(),
				FConcertRequestData::StaticStruct(),
				FConcertResponseData::StaticStruct()
			}; 
			for (TObjectIterator<UStruct> ClassIt; ClassIt; ++ClassIt)
			{
				if (ClassIt->IsChildOf(FConcertMessageData::StaticStruct())
					&& !Excluded.Contains(*ClassIt))
				{
					FString ClassName = ClassIt->GetName();
					ClassName.RemoveFromStart("ConcertAdmin_");
					ClassName.RemoveFromStart("ConcertSession_");
					Result.Add(ClassIt->GetFName(), ClassName);
				}
			}
			return Result;
		}();
		const FString* SanitizedName = CachedSanitizedNames.Find(MessageTypeName);
		return SanitizedName ? *SanitizedName : "UNKNOWN";
	}

	static TSet<FName> FilterAllMessageTypes(TFunctionRef<bool(UStruct*)> ConsumerFunc)
	{
		TSet<FName> Result;
		for (TObjectIterator<UStruct> ClassIt; ClassIt; ++ClassIt)
		{
			if (ClassIt->IsChildOf(FConcertMessageData::StaticStruct())
				&& ConsumerFunc(*ClassIt))
			{
				Result.Add(ClassIt->GetFName());
			}
		}
		return Result;
	}
};
