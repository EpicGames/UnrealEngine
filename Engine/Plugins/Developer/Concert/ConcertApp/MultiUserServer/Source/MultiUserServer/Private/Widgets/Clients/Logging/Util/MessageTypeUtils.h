// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"

namespace UE::MultiUserServer::MessageTypeUtils
{
	/** Gets all message type names (sub-structs of FConcertMessageData - see FConcertLog::MessageTypeName) */
	TSet<FName> GetAllMessageTypeNames();
	
	TSet<FName> GetAllMessageTypeNames_EventsOnly();
	TSet<FName> GetAllMessageTypeNames_RequestsOnly();
	TSet<FName> GetAllMessageTypeNames_ResponseOnlyOnly();
	TSet<FName> GetAllMessageTypeNames_AcksOnlyOnly();

	/** Gets a readable version of a message type name */
	FString SanitizeMessageTypeName(FName MessageTypeName);
};
