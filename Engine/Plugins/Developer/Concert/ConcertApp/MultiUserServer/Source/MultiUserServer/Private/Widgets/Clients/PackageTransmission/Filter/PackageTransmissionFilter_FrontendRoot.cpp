// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageTransmissionFilter_FrontendRoot.h"

#include "Widgets/Clients/PackageTransmission/Filter/PackageTransmissionFrontendFilter_TextSearch.h"


namespace UE::MultiUserServer
{
	TSharedRef<FPackageTransmissionFilter_FrontendRoot> MakeFilter(TSharedRef<FPackageTransmissionEntryTokenizer> Tokenizer)
	{
		TArray<TSharedRef<TConcertFrontendFilter<const FPackageTransmissionEntry&>>> FrontendFilters;
		return MakeShared<FPackageTransmissionFilter_FrontendRoot>(MoveTemp(Tokenizer), FrontendFilters);
	}
}
