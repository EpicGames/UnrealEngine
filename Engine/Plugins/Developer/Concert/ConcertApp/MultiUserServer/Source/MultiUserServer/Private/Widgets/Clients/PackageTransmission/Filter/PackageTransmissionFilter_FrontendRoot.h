// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PackageTransmissionFrontendFilter_TextSearch.h"
#include "Widgets/Util/Filter/ConcertRootFrontendFilter.h"
#include "Widgets/Util/Filter/SConcertFilterBar.h"

class FEndpointToUserNameCache;
class SWidget;

namespace UE::MultiUserServer
{
	class FPackageTransmissionFrontendFilter_TextSearch;
	class FPackageTransmissionEntryTokenizer;

	/** Root filter displayed in package transmission UI */
	class FPackageTransmissionFilter_FrontendRoot : public TConcertFrontendRootFilter<const FPackageTransmissionEntry&, FPackageTransmissionFrontendFilter_TextSearch>
	{
		using Super = TConcertFrontendRootFilter<const FPackageTransmissionEntry&, FPackageTransmissionFrontendFilter_TextSearch>;
	public:

		FPackageTransmissionFilter_FrontendRoot(
			TSharedRef<FPackageTransmissionEntryTokenizer> Tokenizer,
			TArray<TSharedRef<TConcertFrontendFilter<const FPackageTransmissionEntry&>>> FrontendFilters,
			const TArray<TSharedRef<IFilter<const FPackageTransmissionEntry&>>>& NonVisualFilters = {},
			TSharedRef<FFilterCategory> DefaultCategory = MakeShared<FFilterCategory>(NSLOCTEXT("UnrealMultiUserServer.TConcertFrontendRootFilter", "DefaultCategoryLabel", "Default"), FText::GetEmpty())
			)
			: Super(MakeShared<FPackageTransmissionFrontendFilter_TextSearch>(DefaultCategory, MoveTemp(Tokenizer)), MoveTemp(FrontendFilters), NonVisualFilters, DefaultCategory)
		{}
	};

	/** Creates a filter for the global filter log window */
	TSharedRef<FPackageTransmissionFilter_FrontendRoot> MakeFilter(TSharedRef<FPackageTransmissionEntryTokenizer> Tokenizer);
}
