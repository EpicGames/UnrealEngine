// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Clients/PackageTransmission/Util/PackageTransmissionEntryTokenizer.h"
#include "Widgets/Util/Filter/ConcertFrontendFilter.h"
#include "Widgets/Util/Filter/ConcertFrontendFilter_TextSearch.h"

namespace UE::MultiUserServer
{
	class FPackageTransmissionFilter_TextSearch : public TConcertFilter_TextSearch<const FPackageTransmissionEntry&>
	{
	public:
		
		FPackageTransmissionFilter_TextSearch(TSharedRef<FPackageTransmissionEntryTokenizer> Tokenizer)
			: Tokenizer(MoveTemp(Tokenizer))
		{}

	protected:

		virtual void GenerateSearchTerms(const FPackageTransmissionEntry& InItem, TArray<FString>& OutTerms) const override
		{
			OutTerms.Add(Tokenizer->TokenizeTime(InItem));
			OutTerms.Add(Tokenizer->TokenizeOrigin(InItem));
			OutTerms.Add(Tokenizer->TokenizeDestination(InItem));
			OutTerms.Add(Tokenizer->TokenizeSize(InItem));
			OutTerms.Add(Tokenizer->TokenizeRevision(InItem));
			OutTerms.Add(Tokenizer->TokenizePackagePath(InItem));
			OutTerms.Add(Tokenizer->TokenizePackageName(InItem));
		}
	
	private:
		
		/** Helps in converting FConcertLog members into search terms */
		TSharedRef<FPackageTransmissionEntryTokenizer> Tokenizer;
	};
	
    class FPackageTransmissionFrontendFilter_TextSearch : public TConcertFrontendFilter_TextSearch<FPackageTransmissionFilter_TextSearch, const FPackageTransmissionEntry&>
    {
		using Super = TConcertFrontendFilter_TextSearch<FPackageTransmissionFilter_TextSearch, const FPackageTransmissionEntry&>;
    public:
    	
    	FPackageTransmissionFrontendFilter_TextSearch(TSharedPtr<FFilterCategory> InCategory, TSharedRef<FPackageTransmissionEntryTokenizer> Tokenizer)
    		: Super(MoveTemp(InCategory), MoveTemp(Tokenizer))
    	{}
    };
}

