// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilteredPackageTransmissionModel.h"
#include "Widgets/Clients/PackageTransmission/Model/IPackageTransmissionEntrySource.h"

namespace UE::MultiUserServer
{
	FFilteredPackageTransmissionModel::FFilteredPackageTransmissionModel(TSharedRef<IPackageTransmissionEntrySource> InRealSource, TSharedRef<IFilter<const FPackageTransmissionEntry&>> InFilter)
		: RealSource(MoveTemp(InRealSource))
		, Filter(MoveTemp(InFilter))
	{
		RealSource->OnPackageEntriesAdded().AddRaw(this, &FFilteredPackageTransmissionModel::OnSourceEntriesAdded);
		RealSource->OnPackageEntriesModified().AddRaw(this, &FFilteredPackageTransmissionModel::OnSourceEntriesModified);

		Filter->OnChanged().AddRaw(this, &FFilteredPackageTransmissionModel::ReapplyFilter, false);
	}

	FFilteredPackageTransmissionModel::~FFilteredPackageTransmissionModel()
	{
		RealSource->OnPackageEntriesAdded().RemoveAll(this);
		RealSource->OnPackageEntriesModified().RemoveAll(this);

		Filter->OnChanged().RemoveAll(this);
	}

	TSharedPtr<FPackageTransmissionEntry> FFilteredPackageTransmissionModel::GetEntryById(FPackageTransmissionId ID)
	{
		const TOptional<FPackageTransmissionId> Index = TranslatedSourceIndexToFilteredIndex(ID);
		return Index
			? FilteredEntries[*Index]
			: TSharedPtr<FPackageTransmissionEntry>{};
	}

	void FFilteredPackageTransmissionModel::OnSourceEntriesAdded(uint32 NumAdded)
	{
		ApplyFilter(false, RealSource->GetEntries().Num() - NumAdded);
	}

	void FFilteredPackageTransmissionModel::OnSourceEntriesModified(const TSet<FPackageTransmissionId>& ModifiedEntries)
	{
		// Could be more efficient by only looking at ModifiedEntries... but usually the number of items is pretty small
		ReapplyFilter(true);

		TSet<FPackageTransmissionId> FilteredModified;
		FilteredModified.Reserve(ModifiedEntries.Num());
		for (FPackageTransmissionId Id : ModifiedEntries)
		{
			if (Filter->PassesFilter(*RealSource->GetEntryById(Id)))
			{
				FilteredModified.Add(Id);
			}
		}

		if (FilteredModified.Num() > 0)
		{
			OnPackageEntriesModified().Broadcast(FilteredModified);
		}
	}

	TOptional<FPackageTransmissionId> FFilteredPackageTransmissionModel::TranslatedSourceIndexToFilteredIndex(const FPackageTransmissionId OriginalId) const
	{
		const FPackageTransmissionId* ReboundIndex = RebindRealSourceToFilteredIndex.Find(OriginalId);
		return ensure(ReboundIndex && FilteredEntries.IsValidIndex(*ReboundIndex))
			? *ReboundIndex
			: TOptional<FPackageTransmissionId>{};
	}

	void FFilteredPackageTransmissionModel::ReapplyFilter(bool bSilent)
	{
		FilteredEntries.Empty(FilteredEntries.Num());
		RebindRealSourceToFilteredIndex.Empty(RebindRealSourceToFilteredIndex.Num());
		ApplyFilter(bSilent);
	}

	void FFilteredPackageTransmissionModel::ApplyFilter(bool bSilent, int32 StartIndex)
	{
		uint32 NumAdded = 0;

		const TArray<TSharedPtr<FPackageTransmissionEntry>>& Entries = RealSource->GetEntries();
		for (int32 i = StartIndex; i < Entries.Num(); ++i)
		{
			if (Filter->PassesFilter(*Entries[i]))
			{
				FPackageTransmissionId Index = FilteredEntries.Add(Entries[i]);
				RebindRealSourceToFilteredIndex.Add(Entries[i]->TransmissionId, Index);
				++NumAdded;
			}
		}

		if (!bSilent && NumAdded > 0)
		{
			OnPackageEntriesAdded().Broadcast(NumAdded);
		}
	}
}

