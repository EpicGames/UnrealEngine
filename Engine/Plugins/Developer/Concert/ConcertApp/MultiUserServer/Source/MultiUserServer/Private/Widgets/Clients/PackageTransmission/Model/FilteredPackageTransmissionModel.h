// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPackageTransmissionEntrySource.h"
#include "Misc/IFilter.h"
#include "Widgets/Clients/PackageTransmission/Model/PackageTransmissionEntry.h"

template <typename OptionalType> struct TOptional;

namespace UE::MultiUserServer
{
	class FFilteredPackageTransmissionModel : public IPackageTransmissionEntrySource
	{
	public:

		FFilteredPackageTransmissionModel(TSharedRef<IPackageTransmissionEntrySource> RealSource, TSharedRef<IFilter<const FPackageTransmissionEntry&>> Filter);
		virtual ~FFilteredPackageTransmissionModel() override;

		//~ Begin IPackageTransmissionEntrySource Interface
		virtual const TArray<TSharedPtr<FPackageTransmissionEntry>>& GetEntries() override { return FilteredEntries; }
		virtual TSharedPtr<FPackageTransmissionEntry> GetEntryById(FPackageTransmissionId ID) override;
		virtual FOnPackageEntriesAdded& OnPackageEntriesAdded() override { return OnPackageEntriesAddedDelegate; }
		virtual FOnPackageEntriesModified& OnPackageEntriesModified() override { return OnPackageEntriesModifiedDelegate; }
		//~ End IPackageTransmissionEntrySource Interface

	private:

		TSharedRef<IPackageTransmissionEntrySource> RealSource;
		TSharedRef<IFilter<const FPackageTransmissionEntry&>> Filter;

		FOnPackageEntriesAdded OnPackageEntriesAddedDelegate;
		FOnPackageEntriesModified OnPackageEntriesModifiedDelegate;

		TArray<TSharedPtr<FPackageTransmissionEntry>> FilteredEntries;
		TMap<FPackageTransmissionId, FPackageTransmissionId> RebindRealSourceToFilteredIndex;

		void OnSourceEntriesAdded(uint32 NumAdded);
		void OnSourceEntriesModified(const TSet<FPackageTransmissionId>& ModifiedEntries);

		TOptional<FPackageTransmissionId> TranslatedSourceIndexToFilteredIndex(const FPackageTransmissionId OriginalId) const;
		
		void ReapplyFilter(bool bSilent);
		void ApplyFilter(bool bSilent = false, int32 StartIndex = 0);
	};
}

