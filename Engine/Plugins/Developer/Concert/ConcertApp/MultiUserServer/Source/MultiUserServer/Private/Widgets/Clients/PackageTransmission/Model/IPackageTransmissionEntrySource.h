// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PackageTransmissionEntry.h"

namespace UE::MultiUserServer
{
	/** Interfaces for an object that provides access to package transmissions - could be filtered or unfiltered. */
	class IPackageTransmissionEntrySource
	{
	public:

		virtual const TArray<TSharedPtr<FPackageTransmissionEntry>>& GetEntries() = 0;

		/** @return The entry if valid - nullptr if not valid */
		virtual TSharedPtr<FPackageTransmissionEntry> GetEntryById(FPackageTransmissionId ID) = 0;

		/** Special version of OnPackageEntryArrayChanged that is called when entries are added. The last NumEntries number of elements in GetEntries are new. */
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnPackageEntriesAdded, uint32 /*NumEntries*/)
		virtual FOnPackageEntriesAdded& OnPackageEntriesAdded() = 0;
		
		/** Called when the contents of an entry is changes. */
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnPackageEntriesModified, const TSet<FPackageTransmissionId>& /*ModifiedEntries*/);
		virtual FOnPackageEntriesModified& OnPackageEntriesModified() = 0;

		virtual ~IPackageTransmissionEntrySource() = default;
	};
}

