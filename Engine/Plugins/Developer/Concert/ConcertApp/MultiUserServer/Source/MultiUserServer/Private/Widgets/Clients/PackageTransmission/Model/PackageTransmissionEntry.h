// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertWorkspaceData.h"

namespace UE::MultiUserServer
{
	using FPackageTransmissionId = uint64;

	enum class EPackageTransmissionState : uint8
	{
		/** We were told that the transmission has started and waiting for the destination to receive it. */
		InTransmission,
		/** The package was received and the data was not rejected; only the server rejects data. */
		ReceivedAndAccepted,
		/** The server rejected the update (bad request, client had no lock, multiple clients modified concurrently, etc.) */
		Rejected
	};

	enum class EPackageSendDirection : uint8
	{
		ClientToServer,
		ServerToClient
	};
	
	/**
	 * Describes the transmission of a single packet from one network point to another.
	 *
	 * Package transmission to the server happens in two stages:
	 * 1. The client announces it is about to send a package - that's when this entry is created
	 * 2. The server receives the package data - that's when MessageId is set
	 *
	 * Package transmission to the client happens in two stages:
	 * 1. The server sends all package data to the client - that's when this entry is created
	 * 2. The client sends an ACK - that's when MessageID is set
	 */
	struct FPackageTransmissionEntry
	{
		/** Unique ID created by our internal package transmission model - it's internally an array entry (implementation detail -  don't assume it) */
		FPackageTransmissionId TransmissionId;

		FDateTime LocalStartTime = FDateTime::Now(); 
		EPackageTransmissionState TransmissionState = EPackageTransmissionState::InTransmission;
		
		EPackageSendDirection SendDirection;
		
		FConcertPackageInfo PackageInfo;
		uint64 Revision;
		
		/** The origin endpoint announced that the package would have this size */
		uint64 PackageNumBytes;
		
		/** The endpoint ID of the participating client. */
		FGuid ClientEndpointId;

		/**
		 * The ID of the message that causes the package to be acknowledged.
		 * Only set if TransmissionState != EPackageTransmissionState::InTransmission.
		 *
		 * In the case of sending to a client, it is the acknowledgement message.
		 * In the case of receiving from client, it is the message transmitting the sync activity.
		 */
		TOptional<FGuid> MessageId;

		FPackageTransmissionEntry(FPackageTransmissionId TransmissionId, EPackageSendDirection SendDirection, const FConcertPackageInfo& PackageInfo, uint64 Revision, uint64 PackageNumBytes, const FGuid& RemoteEndpointId)
			: TransmissionId(TransmissionId)
			, SendDirection(SendDirection)
			, PackageInfo(PackageInfo)
			, Revision(Revision)
			, PackageNumBytes(PackageNumBytes)
			, ClientEndpointId(RemoteEndpointId)
		{}
	};
}

