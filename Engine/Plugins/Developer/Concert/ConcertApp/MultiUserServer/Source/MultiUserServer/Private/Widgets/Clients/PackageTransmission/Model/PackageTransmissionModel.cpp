// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageTransmissionModel.h"

#include "ConcertLogGlobal.h"
#include "ConcertPackageEvents.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertUtil.h"
#include "ConcertServerUtil.h"
#include "IConcertServer.h"
#include "IConcertSyncServer.h"
#include "IConcertSession.h"
#include "Templates/NonNullPointer.h"

namespace UE::MultiUserServer
{
	FPackageTransmissionModel::FPackageTransmissionModel(TSharedRef<IConcertSyncServer> Server)
		: Server(MoveTemp(Server))
	{
		using namespace ConcertSyncCore;
		
		ConcertPackageEvents::OnLocalBeginSendPackage().AddRaw(this, &FPackageTransmissionModel::OnLocalBeginSendPackage);
		ConcertPackageEvents::OnLocalFinishSendPackage().AddRaw(this, &FPackageTransmissionModel::OnLocalFinishSendPackage);
		ConcertPackageEvents::OnRemoteBeginSendPackage().AddRaw(this, &FPackageTransmissionModel::OnRemoteBeginSendPackage);
		ConcertPackageEvents::OnRemoteFinishSendPackage().AddRaw(this, &FPackageTransmissionModel::OnRemoteFinishSendPackage);
		ConcertPackageEvents::OnRejectRemoteSendPackage().AddRaw(this, &FPackageTransmissionModel::OnRejectRemoteSendPackage);

		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FPackageTransmissionModel::Tick));
	}

	FPackageTransmissionModel::~FPackageTransmissionModel()
	{
		using namespace ConcertSyncCore;
		
		ConcertPackageEvents::OnLocalBeginSendPackage().RemoveAll(this);
		ConcertPackageEvents::OnLocalFinishSendPackage().RemoveAll(this);
		ConcertPackageEvents::OnRemoteBeginSendPackage().RemoveAll(this);
		ConcertPackageEvents::OnRemoteFinishSendPackage().RemoveAll(this);
		ConcertPackageEvents::OnRejectRemoteSendPackage().RemoveAll(this);
		
		FTSTicker::RemoveTicker(TickHandle);
	}

	void FPackageTransmissionModel::OnLocalBeginSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args)
	{
		HandleBeginSend(EPackageSendDirection::ServerToClient, Args);
	}

	void FPackageTransmissionModel::OnLocalFinishSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertFinishSendPackageParams& Args)
	{
		HandleFinishSend(EPackageTransmissionState::ReceivedAndAccepted, Args);
	}

	void FPackageTransmissionModel::OnRemoteBeginSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args)
	{
		HandleBeginSend(EPackageSendDirection::ClientToServer, Args);
	}

	void FPackageTransmissionModel::OnRemoteFinishSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertFinishSendPackageParams& Args)
	{
		HandleFinishSend(EPackageTransmissionState::ReceivedAndAccepted, Args);
	}

	void FPackageTransmissionModel::OnRejectRemoteSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertRejectSendPackageParams& Args)
	{
		HandleFinishSend(EPackageTransmissionState::Rejected, Args);
	}

	void FPackageTransmissionModel::HandleBeginSend(EPackageSendDirection SendDirection, const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args)
	{
		UE_LOGF(LogConcert, Verbose, "OnLocalBeginSendPackage: %ls", *Args.TransmissionId.ToString(EGuidFormats::DigitsWithHyphens));
		check(Args.PackageNumBytes > 0);

		const FPackageTransmissionId ID = Entries.Num();
		Entries.Add(MakeShared<FPackageTransmissionEntry>(ID, SendDirection, Args.PackageInfo, DeterminePackageRevision(SendDirection, Args), Args.PackageNumBytes, Args.RemoteEndpointId));
		InTransmissionToIndex.Add(Args.TransmissionId, ID);
		
		++NumAddedSinceLastTick;
	}

	void FPackageTransmissionModel::HandleFinishSend(EPackageTransmissionState TransmissionState, const ConcertSyncCore::ConcertPackageEvents::FConcertFinishSendPackageParams& Args)
	{
		FPackageTransmissionId Key;
		const bool bHadBeginSendData = InTransmissionToIndex.RemoveAndCopyValue(Args.TransmissionId, Key);
		if (ensure(bHadBeginSendData) && ensure(Entries.IsValidIndex(Key)))
		{
			Entries[Key]->MessageId = Args.MessageId;
			Entries[Key]->TransmissionState = TransmissionState;

			ModifiedSinceLastTick.Add(Key);
		}
	}

	uint64 FPackageTransmissionModel::DeterminePackageRevision(EPackageSendDirection SendDirection, const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args) const
	{
		// Show obviously wrong number
		constexpr uint64 ErrorResult = static_cast<uint64>(-1);
		
		const TSharedPtr<IConcertServerSession> Session = ConcertUtil::GetLiveSessionClientConnectedTo(*Server->GetConcertServer(), Args.RemoteEndpointId);
		if (!Session)
		{
			UE_LOGF(LogConcert, Warning, "Processing package event: failed to get head revision because no session was found for client.");
			return ErrorResult;
		}
		const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = Server->GetLiveSessionDatabase(Session->GetId());
		if (!Database)
		{
			UE_LOGF(LogConcert, Warning, "Processing package event: failed to get head revision because session database could not be accessed.");
			return ErrorResult;
		}

		int64 PackageRevision;
		const bool bCouldGetHeadRevision = (*Database)->GetPackageHeadRevision(Args.PackageInfo.PackageName, PackageRevision);
		if (ensureMsgf(bCouldGetHeadRevision, TEXT("The activity should have been processed properly before this event was emitted")))
		{
			return SendDirection == EPackageSendDirection::ClientToServer
				? PackageRevision + 1	// The moment the client sends to server, the revision is not created, yet. We're computing what the next revision will be.
				: PackageRevision;
		}

		UE_LOGF(LogConcert, Error, "Processing package event: failed to get head revision because the package was not saved in the database.");
		return ErrorResult;
	}

	bool FPackageTransmissionModel::Tick(float)
	{
		// The rational of using Tick is to avoid excessive event calls which could cause filtering and possibly UI operations to rerun often
		
		if (NumAddedSinceLastTick > 0)
		{
			OnPackageEntriesAddedDelegate.Broadcast(NumAddedSinceLastTick);
			NumAddedSinceLastTick = 0;
		}

		if (ModifiedSinceLastTick.Num() > 0)
		{
			OnPackageEntriesModifiedDelegate.Broadcast(ModifiedSinceLastTick);
			ModifiedSinceLastTick.Empty(ModifiedSinceLastTick.Num());
		}

		return true;
	}
}
