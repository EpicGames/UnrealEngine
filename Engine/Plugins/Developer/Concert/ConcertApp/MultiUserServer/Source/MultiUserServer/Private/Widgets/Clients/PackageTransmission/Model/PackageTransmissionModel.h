// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "IPackageTransmissionEntrySource.h"
#include "Widgets/Clients/PackageTransmission/Model/PackageTransmissionEntry.h"

class IConcertSyncServer;

namespace UE::ConcertSyncCore::ConcertPackageEvents
{
	struct FConcertBeginSendPackageParams;
	struct FConcertFinishSendPackageParams;
	struct FConcertRejectSendPackageParams;
}

namespace UE::MultiUserServer
{
	class FPackageTransmissionModel : public IPackageTransmissionEntrySource 
	{
	public:
		
		FPackageTransmissionModel(TSharedRef<IConcertSyncServer> Server);
		virtual ~FPackageTransmissionModel() override;

		//~ Begin IPackageTransmissionEntrySource Interface
		virtual const TArray<TSharedPtr<FPackageTransmissionEntry>>& GetEntries() override { return Entries; }
		virtual TSharedPtr<FPackageTransmissionEntry> GetEntryById(FPackageTransmissionId ID) override { return Entries.IsValidIndex(ID) ? Entries[ID] : TSharedPtr<FPackageTransmissionEntry>{}; }
		virtual FOnPackageEntriesAdded& OnPackageEntriesAdded() override { return OnPackageEntriesAddedDelegate; }
		virtual FOnPackageEntriesModified& OnPackageEntriesModified() override { return OnPackageEntriesModifiedDelegate; }
		//~ End IPackageTransmissionEntrySource Interface

	private:

		TSharedRef<IConcertSyncServer> Server;
		TArray<TSharedPtr<FPackageTransmissionEntry>> Entries;

		FOnPackageEntriesAdded OnPackageEntriesAddedDelegate;
		FOnPackageEntriesModified OnPackageEntriesModifiedDelegate;
		
		FTSTicker::FDelegateHandle TickHandle;
		uint32 NumAddedSinceLastTick = 0;
		TSet<FPackageTransmissionId> ModifiedSinceLastTick;
		TMap<FGuid, FPackageTransmissionId> InTransmissionToIndex;
		
		void OnLocalBeginSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args);
		void OnLocalFinishSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertFinishSendPackageParams& Args);
		void OnRemoteBeginSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args);
		void OnRemoteFinishSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertFinishSendPackageParams& Args);
		void OnRejectRemoteSendPackage(const ConcertSyncCore::ConcertPackageEvents::FConcertRejectSendPackageParams& Args);

		void HandleBeginSend(EPackageSendDirection SendDirection, const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args);
		void HandleFinishSend(EPackageTransmissionState TransmissionState, const ConcertSyncCore::ConcertPackageEvents::FConcertFinishSendPackageParams& Args);

		uint64 DeterminePackageRevision(EPackageSendDirection SendDirection, const ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams& Args) const;
		
		bool Tick(float);
	};
}

