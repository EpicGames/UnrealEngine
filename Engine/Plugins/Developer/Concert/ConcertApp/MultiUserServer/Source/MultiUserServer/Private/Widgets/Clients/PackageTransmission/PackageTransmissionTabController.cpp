// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageTransmissionTabController.h"

#include "ConcertServerStyle.h"
#include "Model/PackageTransmissionModel.h"
#include "SPackageTransmissionView.h"
#include "Util/PackageTransmissionEntryTokenizer.h"

#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FPackageTransmissionTabController"

namespace UE::MultiUserServer
{
	FPackageTransmissionTabController::FPackageTransmissionTabController(
			FName InTabId,
			TSharedRef<FTabManager> InOwningTabManager,
			TSharedRef<FWorkspaceItem> InWorkspaceItem,
			TSharedRef<FPackageTransmissionModel> InTransmissionModel,
			TSharedRef<FEndpointToUserNameCache> InEndpointToUserNameCache,
			FCanScrollToLog InCanScrollToLogDelegate,
			FScrollToLog InScrollToLogDelegate
			)
		: TabId(InTabId)
		, OwningTabManager(MoveTemp(InOwningTabManager))
		, TransmissionModel(MoveTemp(InTransmissionModel))
		, CanScrollToLogDelegate(MoveTemp(InCanScrollToLogDelegate))
		, ScrollToLogDelegate(MoveTemp(InScrollToLogDelegate))
		, Tokenizer(MakeShared<FPackageTransmissionEntryTokenizer>(MoveTemp(InEndpointToUserNameCache)))
	{
		OwningTabManager->RegisterTabSpawner(TabId, FOnSpawnTab::CreateRaw(this, &FPackageTransmissionTabController::SpawnTab))
			// In the future we may create multiple FPackageTransmissionTabController and may want to change this name to depend on some parameter
			.SetDisplayName(LOCTEXT("PackageTabLabel", "Packages"))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FConcertServerStyle::GetStyleSetName(), TEXT("Concert.Icon.Package")));
	}

	FPackageTransmissionTabController::~FPackageTransmissionTabController()
	{
		// Should not be needed because we're most likely destroyed at the same time as when the tab manager is destroyed but let's stay forward-compatible
		OwningTabManager->UnregisterTabSpawner(TabId);
	}

	TSharedRef<SDockTab> FPackageTransmissionTabController::SpawnTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("PackageTabLabel", "Packages"))
			.TabRole(PanelTab)
			[
				SNew(SPackageTransmissionView, TransmissionModel, Tokenizer)
				.CanScrollToLog(CanScrollToLogDelegate)
				.ScrollToLog(ScrollToLogDelegate)
			];
	}
}

#undef LOCTEXT_NAMESPACE
