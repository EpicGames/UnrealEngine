// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Clients/Logging/LogScrollingDelegates.h"

class FEndpointToUserNameCache;
class FSpawnTabArgs;
class FTabManager;
class FWorkspaceItem;
class IConcertSyncServer;
class SDockTab;

namespace UE::MultiUserServer
{
	class FPackageTransmissionEntryTokenizer;
	class FPackageTransmissionModel;

	/** Manages a tab that displays a package transmission model */
	class FPackageTransmissionTabController
	{
	public:
		
		FPackageTransmissionTabController(
			FName TabId,
			TSharedRef<FTabManager> OwningTabManager,
			TSharedRef<FWorkspaceItem> WorkspaceItem,
			TSharedRef<FPackageTransmissionModel> TransmissionModel,
			TSharedRef<FEndpointToUserNameCache> EndpointToUserNameCache,
			FCanScrollToLog CanScrollToLogDelegate,
			FScrollToLog ScrollToLogDelegate
			);
		~FPackageTransmissionTabController();

	private:
		
		FName TabId;
		TSharedRef<FTabManager> OwningTabManager;
		TSharedRef<FPackageTransmissionModel> TransmissionModel;
		
		FCanScrollToLog CanScrollToLogDelegate;
		FScrollToLog ScrollToLogDelegate;
		
		TSharedRef<FPackageTransmissionEntryTokenizer> Tokenizer;

		TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& SpawnTabArgs);
	};
}

