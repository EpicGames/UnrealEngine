// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPackageTransmissionTable.h"

#include "Framework/Views/TableViewMetadata.h"
#include "Model/IPackageTransmissionEntrySource.h"
#include "Settings/MultiUserServerColumnVisibilitySettings.h"
#include "SPackageTransmissionTableFooter.h"
#include "SPackageTransmissionTableRow.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Settings/MultiUserServerPackageTransmissionSettings.h"
#include "Widgets/Clients/PackageTransmission/Model/PackageTransmissionEntry.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SPackageTransmissionTable"

namespace UE::MultiUserServer
{
	void SPackageTransmissionTable::Construct(const FArguments& InArgs, TSharedRef<IPackageTransmissionEntrySource> InPackageEntrySource, TSharedRef<FPackageTransmissionEntryTokenizer> InTokenizer)
	{
		PackageEntrySource = MoveTemp(InPackageEntrySource);
		Tokenizer = MoveTemp(InTokenizer);

		HighlightText = InArgs._HighlightText;
		CanScrollToLogDelegate = InArgs._CanScrollToLog;
		ScrollToLogDelegate = InArgs._ScrollToLog;
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(0, 5, 0, 0)
			[
				CreateTableView()
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SPackageTransmissionTableFooter, PackageEntrySource.ToSharedRef())
				.TotalUnfilteredNum(InArgs._TotalUnfilteredNum)
			]
		];

		UMultiUserServerColumnVisibilitySettings::GetSettings()->OnOnPackageTransmissionColumnVisibilityChanged().AddSP(this, &SPackageTransmissionTable::OnColumnVisibilitySettingsChanged);
		ConcertSharedSlate::RestoreColumnVisibilityState(HeaderRow.ToSharedRef(), UMultiUserServerColumnVisibilitySettings::GetSettings()->GetPackageTransmissionColumnVisibility());

		PackageEntrySource->OnPackageEntriesModified().AddSP(this, &SPackageTransmissionTable::OnPackageEntriesModified);
		PackageEntrySource->OnPackageEntriesAdded().AddSP(this, &SPackageTransmissionTable::OnPackageArrayChanged);
	}

	SPackageTransmissionTable::~SPackageTransmissionTable()
	{
		UMultiUserServerColumnVisibilitySettings::GetSettings()->OnOnPackageTransmissionColumnVisibilityChanged().RemoveAll(this);
		PackageEntrySource->OnPackageEntriesModified().RemoveAll(this);
		PackageEntrySource->OnPackageEntriesAdded().RemoveAll(this);
	}

	TSharedRef<SWidget> SPackageTransmissionTable::CreateViewOptionsButton()
	{
		return ConcertFrontendUtils::CreateViewOptionsComboButton(
			FOnGetContent::CreateLambda([this]()
			{
				FMenuBuilder MenuBuilder(true, nullptr);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("DisplayTimestampInRelativeTime", "Display Relative Time"),
					TAttribute<FText>::CreateLambda([this]()
					{
						const bool bIsVisible = HeaderRow->IsColumnVisible(SPackageTransmissionTableRow::TimeColumn);
						return bIsVisible
							? LOCTEXT("DisplayTimestampInRelativeTime.Tooltip.Visible", "Display the Last Modified column in relative time?")
							: LOCTEXT("DisplayTimestampInRelativeTime.Tooltip.Hidden", "Disabled because the Timestamp column is hidden.");
					}),
					FSlateIcon(),
					FUIAction(
					FExecuteAction::CreateLambda([]()
						{
							UMultiUserServerPackageTransmissionSettings* Settings = UMultiUserServerPackageTransmissionSettings::GetSettings();
				
							switch (Settings->TimestampTimeFormat)
							{
							case ETimeFormat::Relative:
								Settings->TimestampTimeFormat = ETimeFormat::Absolute;
								break;
							case ETimeFormat::Absolute: 
								Settings->TimestampTimeFormat = ETimeFormat::Relative;
								break;
							default:
								checkNoEntry();
							}

							Settings->SaveConfig();
						}),
						FCanExecuteAction::CreateLambda([this] { return HeaderRow->IsColumnVisible(SPackageTransmissionTableRow::TimeColumn); }),
						FIsActionChecked::CreateLambda([this] { return UMultiUserServerPackageTransmissionSettings::GetSettings()->TimestampTimeFormat == ETimeFormat::Relative; })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
				
				return MenuBuilder.MakeWidget();
			}));
	}

	TSharedRef<SWidget> SPackageTransmissionTable::CreateTableView()
	{
		CreateHeaderRow();
		return SAssignNew(TableView, SListView<TSharedPtr<FPackageTransmissionEntry>>)
			.ListItemsSource(&PackageEntrySource->GetEntries())
			.OnContextMenuOpening_Lambda([this]() { return ConcertSharedSlate::MakeTableContextMenu(HeaderRow.ToSharedRef(), GetDefaultColumnVisibilities()); })
			.OnGenerateRow(this, &SPackageTransmissionTable::OnGenerateActivityRowWidget)
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow(HeaderRow);
	}

	TSharedRef<SHeaderRow> SPackageTransmissionTable::CreateHeaderRow()
	{
		HeaderRow = SNew(SHeaderRow)
		.OnHiddenColumnsListChanged_Lambda([this]()
		{
			if (!bIsUpdatingColumnVisibility)
			{
				UMultiUserServerColumnVisibilitySettings::GetSettings()->SetPackageTransmissionColumnVisibility(
				ConcertSharedSlate::SnapshotColumnVisibilityState(HeaderRow.ToSharedRef())
				);
			}
		});

		const TSet<FName> CannotHideColumns = { SPackageTransmissionTableRow::TransmissionStateColumn };
		for (FName ColumnName : SPackageTransmissionTableRow::AllColumns)
		{
			const bool bCannotHide = CannotHideColumns.Contains(ColumnName);
			SHeaderRow::FColumn::FArguments Args = SHeaderRow::FColumn::FArguments()
				.ColumnId(ColumnName)
				.HAlignCell(HAlign_Center)
				.DefaultLabel(SPackageTransmissionTableRow::ColumnsDisplayText[ColumnName]);

			if (bCannotHide)
			{
				Args.ShouldGenerateWidget(bCannotHide);
			}
			
			HeaderRow->AddColumn(Args);
		}

		TGuardValue<bool> DoNotSave(bIsUpdatingColumnVisibility, true);
		RestoreDefaultColumnVisibilities();

		return HeaderRow.ToSharedRef();
	}

	TSharedRef<ITableRow> SPackageTransmissionTable::OnGenerateActivityRowWidget(TSharedPtr<FPackageTransmissionEntry> Item, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return SNew(SPackageTransmissionTableRow, Item, OwnerTable, Tokenizer.ToSharedRef())
			.HighlightText(HighlightText)
			.CanScrollToLog(CanScrollToLogDelegate)
			.ScrollToLog(ScrollToLogDelegate);
	}

	void SPackageTransmissionTable::OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot)
	{
		TGuardValue<bool> GuardValue(bIsUpdatingColumnVisibility, true);
		ConcertSharedSlate::RestoreColumnVisibilityState(HeaderRow.ToSharedRef(), ColumnSnapshot);
	}

	void SPackageTransmissionTable::RestoreDefaultColumnVisibilities() const
	{
		const TMap<FName, bool> DefaultVisibilities = GetDefaultColumnVisibilities();
		for (auto DefaultVisibilitiesIt = DefaultVisibilities.CreateConstIterator(); DefaultVisibilitiesIt; ++DefaultVisibilitiesIt)
		{
			HeaderRow->SetShowGeneratedColumn(DefaultVisibilitiesIt->Key, DefaultVisibilitiesIt->Value);
		}
	}
	
	TMap<FName, bool> SPackageTransmissionTable::GetDefaultColumnVisibilities() const
	{
		TMap<FName, bool> Result;
		const TSet<FName> HiddenByDefault = {
			SPackageTransmissionTableRow::OriginColumn,
			SPackageTransmissionTableRow::DestinationColumn,
			SPackageTransmissionTableRow::PackagePathColumn
		};
		
		for (FName ColumnName : SPackageTransmissionTableRow::AllColumns)
		{
			Result.Add(ColumnName, !HiddenByDefault.Contains(ColumnName));
		}
		return Result;
	}

	void SPackageTransmissionTable::OnPackageEntriesModified(const TSet<FPackageTransmissionId>& Set) const
	{
		TableView->RequestListRefresh();
	}

	void SPackageTransmissionTable::OnPackageArrayChanged(uint32 NumAdded) const
	{
		TableView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
