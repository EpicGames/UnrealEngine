// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/PackageTransmissionEntry.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Clients/Logging/LogScrollingDelegates.h"

class FMenuBuilder;
class ITableRow;
class SHeaderRow;
class STableViewBase;
template<typename T> class SListView;
struct FColumnVisibilitySnapshot;

namespace UE::MultiUserServer
{
	class FPackageTransmissionEntryTokenizer;
	class IPackageTransmissionEntrySource;
	struct FPackageTransmissionEntry;

	/** Displays sent and received packages in a table */
	class SPackageTransmissionTable : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SPackageTransmissionTable)
		{}
			SLATE_ATTRIBUTE(uint32, TotalUnfilteredNum)
			SLATE_ATTRIBUTE(FText, HighlightText)
			SLATE_EVENT(FCanScrollToLog, CanScrollToLog)
			SLATE_EVENT(FScrollToLog, ScrollToLog)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<IPackageTransmissionEntrySource> InPackageEntrySource, TSharedRef<FPackageTransmissionEntryTokenizer> InTokenizer);
		virtual ~SPackageTransmissionTable() override;

		TSharedRef<SWidget> CreateViewOptionsButton();
		
	private:

		TSharedPtr<IPackageTransmissionEntrySource> PackageEntrySource;
		TSharedPtr<FPackageTransmissionEntryTokenizer> Tokenizer;

		TAttribute<FText> HighlightText;
		FCanScrollToLog CanScrollToLogDelegate;
		FScrollToLog ScrollToLogDelegate;
		
		TSharedPtr<SListView<TSharedPtr<FPackageTransmissionEntry>>> TableView;
		TSharedPtr<SHeaderRow> HeaderRow;
		/** Whether we currently loading the column visibility - prevents infinite event recursion */
		bool bIsUpdatingColumnVisibility = false;

		// Table view creation
		TSharedRef<SWidget> CreateTableView();
		TSharedRef<SHeaderRow> CreateHeaderRow();
		TSharedRef<ITableRow> OnGenerateActivityRowWidget(TSharedPtr<FPackageTransmissionEntry> Item, const TSharedRef<STableViewBase>& OwnerTable) const;
		
		void OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot);
		void RestoreDefaultColumnVisibilities() const;
		TMap<FName, bool> GetDefaultColumnVisibilities() const;
		
		void OnPackageEntriesModified(const TSet<FPackageTransmissionId>& Set) const;
		void OnPackageArrayChanged(uint32 NumAdded) const;
	};
}
