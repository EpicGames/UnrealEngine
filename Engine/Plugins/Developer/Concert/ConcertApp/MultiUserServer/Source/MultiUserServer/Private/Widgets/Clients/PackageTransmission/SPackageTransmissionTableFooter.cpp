// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPackageTransmissionTableFooter.h"

#include "Model/IPackageTransmissionEntrySource.h"

#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertTransportLogRow"

namespace UE::MultiUserServer
{
	void SPackageTransmissionTableFooter::Construct(const FArguments& InArgs, TSharedRef<IPackageTransmissionEntrySource> InPackageEntrySource)
	{
		PackageEntrySource = MoveTemp(InPackageEntrySource);
		TotalUnfilteredNum = InArgs._TotalUnfilteredNum;
		
		if (!ensure(TotalUnfilteredNum.IsBound()))
		{
			TotalUnfilteredNum = TAttribute<uint32>::CreateLambda([this](){ return PackageEntrySource->GetEntries().Num(); });
		}
		
		ChildSlot
		[
			SNew(SHorizontalBox)

			// Logs per page text
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return FText::Format(LOCTEXT("DisplayPackagesFmt", "Displaying {0} of {1} packages"), PackageEntrySource->GetEntries().Num(), TotalUnfilteredNum.Get());
				})
			]

			// Gap Filler
			+SHorizontalBox::Slot()
			.FillWidth(1.0)
			[
				SNew(SSpacer)
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE