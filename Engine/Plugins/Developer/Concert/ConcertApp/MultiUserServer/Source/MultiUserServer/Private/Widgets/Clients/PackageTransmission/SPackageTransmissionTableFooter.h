// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class IConcertSyncServer;

namespace UE::MultiUserServer
{
	class IPackageTransmissionEntrySource;

	/** Contains all UI for displaying package transmission, including filters, the table, and view options. */
	class SPackageTransmissionTableFooter : public SCompoundWidget
	{
	public:

		DECLARE_DELEGATE_OneParam(FExtendContextMenu, FMenuBuilder&)

		SLATE_BEGIN_ARGS(SPackageTransmissionTableFooter)
		{}
			SLATE_EVENT(FExtendContextMenu, ExtendViewOptions)
			SLATE_ATTRIBUTE(uint32, TotalUnfilteredNum)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<IPackageTransmissionEntrySource> InPackageEntrySource);

	private:
		
		TSharedPtr<IPackageTransmissionEntrySource> PackageEntrySource;
		TAttribute<uint32> TotalUnfilteredNum;
	};
}
