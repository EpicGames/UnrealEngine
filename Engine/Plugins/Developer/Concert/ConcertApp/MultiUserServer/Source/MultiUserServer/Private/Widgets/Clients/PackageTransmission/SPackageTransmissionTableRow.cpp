// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPackageTransmissionTableRow.h"

#include "ConcertServerStyle.h"
#include "Model/PackageTransmissionEntry.h"
#include "Util/PackageTransmissionEntryTokenizer.h"

#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SPackageTransmissionTableRow"

namespace UE::MultiUserServer
{
	const FName SPackageTransmissionTableRow::TimeColumn("TimeColumn");
	const FName SPackageTransmissionTableRow::OriginColumn("OriginColumn");
	const FName SPackageTransmissionTableRow::DestinationColumn("DestinationColumn");
	const FName SPackageTransmissionTableRow::SizeColumn("SizeColumn");
	const FName SPackageTransmissionTableRow::RevisionColumn("VersionColumn");
	const FName SPackageTransmissionTableRow::PackagePathColumn("PackagePathColumn");
	const FName SPackageTransmissionTableRow::PackageNameColumn("PackageNameColumn");
	const FName SPackageTransmissionTableRow::TransmissionStateColumn("TransmissionStateColumn");
	
	const TArray<FName> SPackageTransmissionTableRow::AllColumns({ TimeColumn, OriginColumn, DestinationColumn, SizeColumn, RevisionColumn, PackagePathColumn, PackageNameColumn, TransmissionStateColumn });
	const TMap<FName, FText> SPackageTransmissionTableRow::ColumnsDisplayText({
		{ TimeColumn, LOCTEXT("TimeColumn", "Time") },
		{ OriginColumn, LOCTEXT("OriginColumn", "Origin") },
		{ DestinationColumn, LOCTEXT("DestinationColumn", "Destination") },
		{ SizeColumn, LOCTEXT("SizeColumn", "Size") },
		{ RevisionColumn, LOCTEXT("RevisionColumn", "Revision") },
		{ PackagePathColumn, LOCTEXT("PackagePathColumn", "Package Path") },
		{ PackageNameColumn, LOCTEXT("PackageNameColumn", "Package Name") },
		{ TransmissionStateColumn, LOCTEXT("TransmissionStateColumn", "State") }
	});
	
	void SPackageTransmissionTableRow::Construct(const FArguments& InArgs, TSharedPtr<FPackageTransmissionEntry> InPackageEntry, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<FPackageTransmissionEntryTokenizer> InTokenizer)
	{
		PackageEntry = MoveTemp(InPackageEntry);
		Tokenizer = MoveTemp(InTokenizer);

		HighlightText = InArgs._HighlightText;
		CanScrollToLogDelegate = InArgs._CanScrollToLog;
		ScrollToLogDelegate = InArgs._ScrollToLog;
		
		SMultiColumnTableRow<TSharedPtr<FPackageTransmissionEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	TSharedRef<SWidget> SPackageTransmissionTableRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		if (ColumnName == TransmissionStateColumn)
		{
			return CreateTransmissionStateColumn();
		}

		return SNew(STextBlock)
			.HighlightText(HighlightText)
			.Text_Lambda([this, ColumnName]()
			{
				using FColumnWidgetFactoryFunc = FString(FPackageTransmissionEntryTokenizer::*)(const FPackageTransmissionEntry&) const;
				const TMap<FName, FColumnWidgetFactoryFunc> OverrideFactories = {
					{ TimeColumn, &FPackageTransmissionEntryTokenizer::TokenizeTime },
					{ OriginColumn, &FPackageTransmissionEntryTokenizer::TokenizeOrigin },
					{ DestinationColumn, &FPackageTransmissionEntryTokenizer::TokenizeDestination },
					{ SizeColumn, &FPackageTransmissionEntryTokenizer::TokenizeSize },
					{ RevisionColumn, &FPackageTransmissionEntryTokenizer::TokenizeRevision },
					{ PackagePathColumn, &FPackageTransmissionEntryTokenizer::TokenizePackagePath },
					{ PackageNameColumn, &FPackageTransmissionEntryTokenizer::TokenizePackageName }
				};
				
				return FText::FromString(Invoke(OverrideFactories[ColumnName], *Tokenizer, *PackageEntry));
			});
	}
	

	TSharedRef<SWidget> SPackageTransmissionTableRow::CreateTransmissionStateColumn() const
	{
		return SNew(SOverlay)

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateTransmissionInProgressWidget()
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateTransmissionSuccessWidget()
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateTransmissionFailureWidget()
			];
	}

	TSharedRef<SWidget> SPackageTransmissionTableRow::CreateTransmissionInProgressWidget() const
	{
		return SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SThrobber)
				.Animate(SThrobber::VerticalAndOpacity)
				.Visibility_Lambda([this]()
				{
					return PackageEntry->TransmissionState == EPackageTransmissionState::InTransmission
						? EVisibility::Visible
						: EVisibility::Hidden;
				})
				.ToolTipText(LOCTEXT("Intransmission", "Transmitting..."))
			];
	}

	TSharedRef<SWidget> SPackageTransmissionTableRow::CreateTransmissionSuccessWidget() const
	{
		return SharedCreateTransmissionCompletedWidget(EPackageTransmissionState::ReceivedAndAccepted, FConcertServerStyle::Get().GetBrush("Concert.PackageTransmission.Success"));
	}

	TSharedRef<SWidget> SPackageTransmissionTableRow::CreateTransmissionFailureWidget() const
	{
		return SharedCreateTransmissionCompletedWidget(EPackageTransmissionState::Rejected, FConcertServerStyle::Get().GetBrush("Concert.PackageTransmission.Failure"));
	}

	TSharedRef<SWidget> SPackageTransmissionTableRow::SharedCreateTransmissionCompletedWidget(EPackageTransmissionState AllowedTransmissionState, const FSlateBrush* ImageBrush) const
	{
		return SNew(SButton)
			.ContentPadding(0.f)
			.IsEnabled_Lambda([this]()
			{
				return CanScrollToLog();
			})
			.OnPressed_Lambda([this]()
			{
				ScrollToLog();
			})
			.Visibility_Lambda([this, AllowedTransmissionState]()
			{
				return PackageEntry->TransmissionState == AllowedTransmissionState
					? EVisibility::Visible
					: EVisibility::Hidden;
			})
			.ToolTipText_Lambda([this, AllowedTransmissionState]()
			{
				const FText BaseTooltip = PackageEntry->TransmissionState == AllowedTransmissionState
					? LOCTEXT("PackageTransmission.Accepted", "Package received.")
					: LOCTEXT("PackageTransmission.Rejected", "Package was rejected by the server.");

				FText ErrorMessage;
				const bool bCanScroll = CanScrollToLog(ErrorMessage);
				return bCanScroll
					? BaseTooltip
					: FText::Format(LOCTEXT("PackageTransmissionFmt", "Cannot scroll to log: {0}\n{1}"), ErrorMessage, BaseTooltip);
			})
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
				.Image(ImageBrush)
			];
	}
	
	bool SPackageTransmissionTableRow::CanScrollToLog(FText& ErrorMessage) const
	{
		return PackageEntry->MessageId && CanScrollToLogDelegate.Execute(*PackageEntry->MessageId, [this](const FConcertLogEntry& Entry)
		{
			return SharedFilterLogEntry(Entry);
		}, ErrorMessage);
	}

	void SPackageTransmissionTableRow::ScrollToLog() const
	{
		ScrollToLogDelegate.Execute(*PackageEntry->MessageId, [this](const FConcertLogEntry& Entry)
		{
			return SharedFilterLogEntry(Entry);
		});
	}
	
	bool SPackageTransmissionTableRow::SharedFilterLogEntry(const FConcertLogEntry& Entry) const
	{
		return Entry.Log.MessageAction == EConcertLogMessageAction::Process;
	}
}

#undef LOCTEXT_NAMESPACE
