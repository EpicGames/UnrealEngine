// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Clients/Logging/LogScrollingDelegates.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;
namespace UE::MultiUserServer { struct FPackageTransmissionEntry; }
struct FConcertLogEntry;

class FConcertLogTokenizer;

namespace UE::MultiUserServer
{
	class FPackageTransmissionEntryTokenizer;
	enum class EPackageTransmissionState : uint8;

	class SPackageTransmissionTableRow : public SMultiColumnTableRow<TSharedPtr<FPackageTransmissionEntry>>
	{
	public:

		static const FName TimeColumn;
		static const FName OriginColumn;
		static const FName DestinationColumn;
		static const FName SizeColumn;
		static const FName RevisionColumn;
		static const FName PackagePathColumn;
		static const FName PackageNameColumn;
		static const FName TransmissionStateColumn;

		static const TArray<FName> AllColumns;
		static const TMap<FName, FText> ColumnsDisplayText;
		
		SLATE_BEGIN_ARGS(SPackageTransmissionTableRow)
		{}
			SLATE_ATTRIBUTE(FText, HighlightText)
			SLATE_EVENT(FCanScrollToLog, CanScrollToLog)
			SLATE_EVENT(FScrollToLog, ScrollToLog)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<FPackageTransmissionEntry> InPackageEntry, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<FPackageTransmissionEntryTokenizer> InTokenizer);

		//~ Begin SMultiColumnTableRow Interface
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
		//~ End SMultiColumnTableRow Interface
		
	private:

		TSharedPtr<FPackageTransmissionEntry> PackageEntry;
		TSharedPtr<FPackageTransmissionEntryTokenizer> Tokenizer;

		TAttribute<FText> HighlightText;
		FCanScrollToLog CanScrollToLogDelegate;
		FScrollToLog ScrollToLogDelegate;
		
		TSharedRef<SWidget> CreateTransmissionStateColumn() const;
		
		TSharedRef<SWidget> CreateTransmissionInProgressWidget() const;
		TSharedRef<SWidget> CreateTransmissionSuccessWidget() const;
		TSharedRef<SWidget> CreateTransmissionFailureWidget() const;
		TSharedRef<SWidget> SharedCreateTransmissionCompletedWidget(EPackageTransmissionState AllowedTransmissionState, const FSlateBrush* ImageBrush) const;

		bool CanScrollToLog() const { FText Dummy; return CanScrollToLog(Dummy); }
		bool CanScrollToLog(FText& ErrorMessage) const;
		void ScrollToLog() const;
		bool SharedFilterLogEntry(const FConcertLogEntry& Entry) const;
	};
}

