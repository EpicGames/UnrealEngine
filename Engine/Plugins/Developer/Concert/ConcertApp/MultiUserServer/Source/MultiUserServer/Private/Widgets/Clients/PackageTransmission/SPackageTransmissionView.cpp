// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPackageTransmissionView.h"

#include "SPackageTransmissionTable.h"
#include "Filter/PackageTransmissionFrontendFilter_TextSearch.h"
#include "Filter/PackageTransmissionFilter_FrontendRoot.h"
#include "Model/FilteredPackageTransmissionModel.h"

#include "Widgets/Clients/PackageTransmission/Model/IPackageTransmissionEntrySource.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SPackageTransmissionView"

namespace UE::MultiUserServer
{
	void SPackageTransmissionView::Construct(const FArguments& InArgs, TSharedRef<IPackageTransmissionEntrySource> InPackageEntrySource, TSharedRef<FPackageTransmissionEntryTokenizer> InTokenizer)
	{
		PackageEntrySource = MoveTemp(InPackageEntrySource);
		RootFilter = MakeFilter(InTokenizer);
		FilteredModel = MakeShared<FFilteredPackageTransmissionModel>(PackageEntrySource.ToSharedRef(), RootFilter.ToSharedRef());

		const TSharedRef<SPackageTransmissionTable> Table = SNew(SPackageTransmissionTable, FilteredModel.ToSharedRef(), InTokenizer)
			.HighlightText_Lambda([this](){ return RootFilter->GetTextSearchFilter()->GetSearchText(); })
			.CanScrollToLog(InArgs._CanScrollToLog)
			.ScrollToLog(InArgs._ScrollToLog)
			.TotalUnfilteredNum_Lambda([this](){ return PackageEntrySource->GetEntries().Num(); });
		
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.BorderBackgroundColor(FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f)))
			.Padding(2)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					RootFilter->BuildFilterWidgets
					(
						FFilterWidgetArgs().RightOfSearchBar
						(
							Table->CreateViewOptionsButton()
						)
					)
				]
				
				+SVerticalBox::Slot()
				[
					Table
				]
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE 
