// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Clients/Logging/LogScrollingDelegates.h"

class IConcertSyncServer;

namespace UE::MultiUserServer
{
	class FFilteredPackageTransmissionModel;
	class FPackageTransmissionEntryTokenizer;
	class FPackageTransmissionFilter_FrontendRoot;
	class IPackageTransmissionEntrySource;

	/** Contains all UI for displaying package transmission, including filters, the table, and view options. */
	class SPackageTransmissionView : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SPackageTransmissionView)
		{}
			SLATE_EVENT(FCanScrollToLog, CanScrollToLog)
			SLATE_EVENT(FScrollToLog, ScrollToLog)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<IPackageTransmissionEntrySource> InPackageEntrySource, TSharedRef<FPackageTransmissionEntryTokenizer> InTokenizer);

	private:

		TSharedPtr<IPackageTransmissionEntrySource> PackageEntrySource;
		
		TSharedPtr<FPackageTransmissionFilter_FrontendRoot> RootFilter;
		TSharedPtr<FFilteredPackageTransmissionModel> FilteredModel;
		
		FCanScrollToLog CanScrollToLogDelegate;
		FScrollToLog ScrollToLogDelegate;
		
		TSharedRef<SWidget> CreateOptionsButton();
		TSharedRef<SWidget> CreateOptionsButtonMenu();
	};
}
