// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageTransmissionEntryTokenizer.h"

#include "Math/UnitConversion.h"
#include "Widgets/Clients/PackageTransmission/Model/PackageTransmissionEntry.h"
#include "Settings/MultiUserServerPackageTransmissionSettings.h"
#include "Widgets/Clients/Util/EndpointToUserNameCache.h"

namespace UE::MultiUserServer
{
	FPackageTransmissionEntryTokenizer::FPackageTransmissionEntryTokenizer(TSharedRef<FEndpointToUserNameCache> EndpointInfoGetter)
		: EndpointInfoGetter(MoveTemp(EndpointInfoGetter))
	{}

	FString FPackageTransmissionEntryTokenizer::TokenizeTime(const FPackageTransmissionEntry& Entry) const
	{
		const FDateTime LocalNow = FDateTime::Now();
		return ConcertFrontendUtils::FormatTime(Entry.LocalStartTime, UMultiUserServerPackageTransmissionSettings::GetSettings()->TimestampTimeFormat, &LocalNow).ToString();
	}

	FString FPackageTransmissionEntryTokenizer::TokenizeOrigin(const FPackageTransmissionEntry& Entry) const
	{
		switch (Entry.SendDirection)
		{
		case EPackageSendDirection::ClientToServer: return EndpointInfoGetter->GetEndpointDisplayString(Entry.ClientEndpointId);
		case EPackageSendDirection::ServerToClient: return TEXT("Server");
		default:
			checkNoEntry();
			return TEXT("n/a");
		}
	}

	FString FPackageTransmissionEntryTokenizer::TokenizeDestination(const FPackageTransmissionEntry& Entry) const
	{
		switch (Entry.SendDirection)
		{
		case EPackageSendDirection::ServerToClient: return EndpointInfoGetter->GetEndpointDisplayString(Entry.ClientEndpointId);
		case EPackageSendDirection::ClientToServer: return TEXT("Server");
		default:
			checkNoEntry();
			return TEXT("n/a");
		}
	}

	FString FPackageTransmissionEntryTokenizer::TokenizeSize(const FPackageTransmissionEntry& Entry) const
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(Entry.PackageNumBytes, EUnit::Bytes);
		return FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));
	}

	FString FPackageTransmissionEntryTokenizer::TokenizeRevision(const FPackageTransmissionEntry& Entry) const
	{
		return FString::FromInt(Entry.Revision);
	}

	FString FPackageTransmissionEntryTokenizer::TokenizePackagePath(const FPackageTransmissionEntry& Entry) const
	{
		return Entry.PackageInfo.PackageName.ToString();
	}

	FString FPackageTransmissionEntryTokenizer::TokenizePackageName(const FPackageTransmissionEntry& Entry) const
	{
		FString PackagePath = Entry.PackageInfo.PackageName.ToString();
		int32 Index;
		if (PackagePath.FindLastChar(TEXT('/'), Index) && ensure(Index != PackagePath.Len() - 1))
		{
			PackagePath.RightChopInline(Index + 1);
		}
		return PackagePath;
	}
}
