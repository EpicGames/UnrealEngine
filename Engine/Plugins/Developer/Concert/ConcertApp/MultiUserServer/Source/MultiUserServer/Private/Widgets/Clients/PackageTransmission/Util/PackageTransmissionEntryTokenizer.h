// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FEndpointToUserNameCache;

namespace UE::MultiUserServer
{
	struct FPackageTransmissionEntry;
	
	/** Helps with printing FPackageTransmissionEntry to text so text search can search exactly what is displayed in the UI. */
	class FPackageTransmissionEntryTokenizer
	{
	public:
		
		FPackageTransmissionEntryTokenizer(TSharedRef<FEndpointToUserNameCache> EndpointInfoGetter);

		FString TokenizeTime(const FPackageTransmissionEntry& Entry) const;
		FString TokenizeOrigin(const FPackageTransmissionEntry& Entry) const;
		FString TokenizeDestination(const FPackageTransmissionEntry& Entry) const;
		FString TokenizeSize(const FPackageTransmissionEntry& Entry) const;
		FString TokenizeRevision(const FPackageTransmissionEntry& Entry) const;
		FString TokenizePackagePath(const FPackageTransmissionEntry& Entry) const;
		FString TokenizePackageName(const FPackageTransmissionEntry& Entry) const;

	private:
		
		/** Used so we can look up client and server info (even after client has disconnected) */
		TSharedRef<FEndpointToUserNameCache> EndpointInfoGetter;
	};
}


