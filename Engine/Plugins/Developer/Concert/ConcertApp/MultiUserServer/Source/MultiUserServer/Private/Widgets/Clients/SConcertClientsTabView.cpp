// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertClientsTabView.h"

#include "ConcertServerStyle.h"
#include "IConcertModule.h"
#include "IConcertSyncServer.h"
#include "Logging/Filter/ConcertLogFilter_FrontendRoot.h"
#include "Logging/Source/GlobalLogSource.h"
#include "Logging/Util/ConcertLogTokenizer.h"
#include "PackageTransmission/PackageTransmissionTabController.h"
#include "PackageTransmission/Model/PackageTransmissionModel.h"
#include "Util/EndpointToUserNameCache.h"
#include "Widgets/Clients/Browser/Models/IClientNetworkStatisticsModel.h"
#include "Widgets/Clients/Browser/SConcertNetworkBrowser.h"
#include "Widgets/Clients/Browser/Models/ClientBrowserModel.h"
#include "Widgets/Clients/Browser/Models/ClientNetworkStatisticsModel.h"
#include "Widgets/Clients/Logging/SConcertTransportLog.h"

#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertClientsTabView"

namespace UE::MultiUserServer
{
	const FName SConcertClientsTabView::ClientBrowserTabId("ClientBrowserTabId");
	const FName SConcertClientsTabView::GlobalLogTabId("GlobalLogTabId");
	const FName SConcertClientsTabView::PackageTransmissionTabId("PackageTransmissionTabId");

	void SConcertClientsTabView::Construct(const FArguments& InArgs, FName InStatusBarID, TSharedRef<IConcertSyncServer> InServer, TSharedRef<FGlobalLogSource> InLogBuffer)
	{
		Server = MoveTemp(InServer);
		LogBuffer = MoveTemp(InLogBuffer);
		ClientInfoCache = MakeShared<FEndpointToUserNameCache>(Server->GetConcertServer());
		LogTokenizer = MakeShared<FConcertLogTokenizer>(ClientInfoCache.ToSharedRef());
		PackageTransmissionModel = MakeShared<FPackageTransmissionModel>(Server.ToSharedRef());
		
		SConcertTabViewWithManagerBase::Construct(
			SConcertTabViewWithManagerBase::FArguments()
			.ConstructUnderWindow(InArgs._ConstructUnderWindow)
			.ConstructUnderMajorTab(InArgs._ConstructUnderMajorTab)
			.CreateTabs(FCreateTabs::CreateLambda([this, &InArgs](const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				CreateTabs(InTabManager, InLayout, InArgs);
			}))
			.LayoutName("ConcertClientsTabView_v0.2"),
			InStatusBarID
		);
	}

	void SConcertClientsTabView::ShowConnectedClients(const FGuid& SessionId) const
	{
		ClientBrowser->ShowOnlyClientsFromSession(SessionId);
	}

	void SConcertClientsTabView::OpenClientLogTab(const FGuid& ClientMessageNodeId) const
	{
		const FName TabId = *ClientMessageNodeId.ToString();
		if (const TSharedPtr<SDockTab> ExistingTab = GetTabManager()->FindExistingLiveTab(FTabId(TabId)))
		{
			GetTabManager()->DrawAttention(ExistingTab.ToSharedRef());
		}
		else
		{
			const TOptional<FConcertClientInfo> ClientInfo = ClientInfoCache->GetClientInfoFromNodeId(ClientMessageNodeId);
			const TSharedRef<SDockTab> NewTab = SNew(SDockTab)
				.Label_Lambda([this, ClientMessageNodeId]()
				{
					const TOptional<FConcertClientInfo> ClientInfo = ClientInfoCache->GetClientInfoFromNodeId(ClientMessageNodeId);
					return ClientInfo
						? FText::Format(LOCTEXT("ClientTabFmt", "{0} Log"), FText::FromString(ClientInfo->DisplayName))
						: FText::FromString(ClientMessageNodeId.ToString(EGuidFormats::DigitsWithHyphens));
				})
				.ToolTipText(FText::Format(LOCTEXT("ClientTabTooltipFmt", "Logs all networked requests originating or going to client {0} (NodeId = {1})"), ClientInfo ? FText::FromString(ClientInfo->DisplayName) : FText::GetEmpty(), FText::FromString(ClientMessageNodeId.ToString())))
				.TabRole(PanelTab)
				[
					SNew(SConcertTransportLog, LogBuffer.ToSharedRef(), ClientInfoCache.ToSharedRef(), LogTokenizer.ToSharedRef())
					.Filter(UE::MultiUserServer::MakeClientLogFilter(LogTokenizer.ToSharedRef(), ClientMessageNodeId, ClientInfoCache.ToSharedRef()))
				];

			NewTab->SetTabIcon(FConcertServerStyle::Get().GetBrush(TEXT("Concert.Icon.LogSession")));

			// We need a tab to place the client tab next to
			if (IsGlobalLogOpen())
			{
				const FTabManager::FLiveTabSearch Search(GlobalLogTabId);
				GetTabManager()->InsertNewDocumentTab(TabId, Search, NewTab);
			}
			else
			{
				OpenGlobalLogTab();
				
				const FTabManager::FLiveTabSearch Search(GlobalLogTabId);
				GetTabManager()->InsertNewDocumentTab(TabId, Search, NewTab);

				CloseGlobalLogTab();
			}
		}
	}

	bool SConcertClientsTabView::CanScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc, FText& ErrorMessage) const
	{
		// GlobalTransportLog can be nullptr if the tab has not (yet) been created
		if (!GlobalTransportLog)
		{
			return false;
		}
		
		const bool bCanScrollToLog = GlobalTransportLog->CanScrollToLog(MessageId, FilterFunc);
		if (!bCanScrollToLog)
		{
			ErrorMessage = LOCTEXT("ScrollToLog.LogNotAvailable", "The log is filtered out.");
			return false;
		}

		return true;
	}

	void SConcertClientsTabView::ScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc) const
	{
		OpenGlobalLogTab();
		
		if (ensureMsgf(GlobalTransportLog, TEXT("OpenGlobalLogTab failed to create the global log widget. Investigate.")))
		{
			GlobalTransportLog->ScrollToLog(MessageId, FilterFunc);
		}
	}

	void SConcertClientsTabView::OpenGlobalLogTab() const
	{
		GetTabManager()->TryInvokeTab(GlobalLogTabId);
	}

	void SConcertClientsTabView::CloseGlobalLogTab() const
	{
		if (const TSharedPtr<SDockTab> GlobalLogTab = GetGlobalLogTab())
		{
			GlobalLogTab->RequestCloseTab();
		}
	}

	bool SConcertClientsTabView::IsGlobalLogOpen() const
	{
		return GetGlobalLogTab().IsValid();
	}

	TSharedPtr<SDockTab> SConcertClientsTabView::GetGlobalLogTab() const
	{
		return GetTabManager()->FindExistingLiveTab(GlobalLogTabId);
	}

	void SConcertClientsTabView::CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FArguments& InArgs)
	{
		const TSharedRef<FWorkspaceItem> WorkspaceItem = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("ClientBrowser", "Client Browser"));
		
		MainPackageTransmissionTab = MakeShared<FPackageTransmissionTabController>(
			PackageTransmissionTabId,
			InTabManager,
			WorkspaceItem,
			PackageTransmissionModel.ToSharedRef(),
			ClientInfoCache.ToSharedRef(),
			FCanScrollToLog::CreateSP(this, &SConcertClientsTabView::CanScrollToLog),
			FScrollToLog::CreateSP(this, &SConcertClientsTabView::ScrollToLog)
			);
		InTabManager->RegisterTabSpawner(ClientBrowserTabId, FOnSpawnTab::CreateSP(this, &SConcertClientsTabView::SpawnClientBrowserTab))
			.SetDisplayName(LOCTEXT("ClientBrowserTabLabel", "Clients"))
			.SetGroup(WorkspaceItem)
			.SetIcon(FSlateIcon(FConcertServerStyle::GetStyleSetName(), TEXT("Concert.Icon.Client")));
		InTabManager->RegisterTabSpawner(GlobalLogTabId, FOnSpawnTab::CreateSP(this, &SConcertClientsTabView::SpawnGlobalLogTab))
			.SetDisplayName(LOCTEXT("ServerLogTabLabel", "Server Log"))
			.SetGroup(WorkspaceItem)
			.SetIcon(FSlateIcon(FConcertServerStyle::GetStyleSetName(), TEXT("Concert.Icon.LogServer")));
		
		InLayout->AddArea
			(
				FTabManager::NewPrimaryArea()
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(0.5f)
						->SetOrientation(Orient_Horizontal)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.65f)
							->AddTab(ClientBrowserTabId, ETabState::OpenedTab)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.35f)
							->AddTab(PackageTransmissionTabId, ETabState::OpenedTab)
								
						)
					)
					->Split
					(
					FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(GlobalLogTabId, ETabState::OpenedTab)
					)
			);
	}

	TSharedRef<SDockTab> SConcertClientsTabView::SpawnClientBrowserTab(const FSpawnTabArgs& InTabArgs)
	{
		using namespace UE::MultiUserServer;
		const TSharedRef<FClientNetworkStatisticsModel> NetworkStatisticsModel = MakeShared<FClientNetworkStatisticsModel>();
		
		return SNew(SDockTab)
			.Label(LOCTEXT("ClientBrowserTabLabel", "Clients"))
			.TabRole(PanelTab)
			[
				SAssignNew(ClientBrowser, UE::MultiUserServer::SConcertNetworkBrowser,
					MakeShared<UE::MultiUserServer::FClientBrowserModel>(Server->GetConcertServer(), ClientInfoCache.ToSharedRef(), NetworkStatisticsModel), NetworkStatisticsModel, Server->GetConcertServer())
				.RightOfSearch()
				[
					CreateOpenGlobalLogButton()
				]
				.OnServerDoubleClicked_Raw(this, &SConcertClientsTabView::OpenGlobalLogTab)
				.OnClientDoubleClicked_Raw(this, &SConcertClientsTabView::OpenClientLogTab)
			]; 
	}

	TSharedRef<SDockTab> SConcertClientsTabView::SpawnGlobalLogTab(const FSpawnTabArgs& InTabArgs)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("ServerLogTabLabel", "Server Log"))
			.TabRole(PanelTab)
			[
				SAssignNew(GlobalTransportLog, SConcertTransportLog, LogBuffer.ToSharedRef(), ClientInfoCache.ToSharedRef(), LogTokenizer.ToSharedRef())
				.Filter(UE::MultiUserServer::MakeGlobalLogFilter(LogTokenizer.ToSharedRef()))
			]; 
	}

	TSharedRef<SWidget> SConcertClientsTabView::CreateOpenGlobalLogButton() const
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("OpenGlobalLogTooltip", "Opens the Server Log which logs all incoming networked messages."))
			.ContentPadding(FMargin(1, 0))
			.Visibility_Lambda(
				[this]()
				{
					const bool bIsGlobalLogOpen = IsGlobalLogOpen();
					return bIsGlobalLogOpen ? EVisibility::Collapsed : EVisibility::Visible;
				})
			.OnClicked_Lambda([this]()
			{
				OpenGlobalLogTab();
				return FReply::Handled();
			})
			[
				SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0, 0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FConcertServerStyle::Get().GetBrush(TEXT("Concert.Icon.LogServer")))
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0, 0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OpenServerLog", "Open Server Log"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
			];
	}
}

#undef LOCTEXT_NAMESPACE
