// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/ConcertLogEntry.h"
#include "Widgets/SConcertTabViewWithManagerBase.h"

class FConcertClientsTabController;
class FEndpointToUserNameCache;
class FGlobalLogSource;
class FTabManager;
class IConcertSyncServer;
class SDockTab;
class SPromptConcertLoggingEnabled;
class SWidget;
class SWindow;
struct FConcertClientInfo;

namespace UE::MultiUserServer
{
	class FConcertLogTokenizer;
	class FPackageTransmissionTabController;
	class FPackageTransmissionModel;
	class SConcertNetworkBrowser;
	class SConcertTransportLog;
	
	/** Manages the UI logic of the Clients tab */
	class SConcertClientsTabView : public SConcertTabViewWithManagerBase
	{
	public:
		
		static const FName ClientBrowserTabId;
		static const FName GlobalLogTabId;
		static const FName PackageTransmissionTabId;

		SLATE_BEGIN_ARGS(SConcertClientsTabView)
		{}
			SLATE_ARGUMENT(TSharedPtr<SDockTab>, ConstructUnderMajorTab)
			SLATE_ARGUMENT(TSharedPtr<SWindow>, ConstructUnderWindow)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FName InStatusBarID, TSharedRef<IConcertSyncServer> InServer, TSharedRef<FGlobalLogSource> InLogBuffer);
		
		void ShowConnectedClients(const FGuid& SessionId) const;
		void OpenGlobalLogTab() const;
		void CloseGlobalLogTab() const;
		void OpenClientLogTab(const FGuid& ClientMessageNodeId) const;

		/** Whether it is possible to call ScrollToLog. */
		bool CanScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc, FText& ErrorMessage) const;
		/** Scrolls to the log in the server log tab. */
		void ScrollToLog(const FGuid& MessageId, FConcertLogEntryFilterFunc FilterFunc) const;
		
		bool IsGlobalLogOpen() const;
		TSharedPtr<SDockTab> GetGlobalLogTab() const;
		
	private:

		/** Used to look up client info */
		TSharedPtr<IConcertSyncServer> Server;
		/** Buffers all logs globally */
		TSharedPtr<FGlobalLogSource> LogBuffer;

		/** Caches client info so it remains available even after a client disconnects */
		TSharedPtr<FEndpointToUserNameCache> ClientInfoCache;
		/** Used by various systems to convert logs to text */
		TSharedPtr<FConcertLogTokenizer> LogTokenizer;
		
		/** Knows about when packages are sent and received by the server */
		TSharedPtr<FPackageTransmissionModel> PackageTransmissionModel;
		TSharedPtr<FPackageTransmissionTabController> MainPackageTransmissionTab;

		TSharedPtr<SConcertNetworkBrowser> ClientBrowser;

		/** The widget inside of the server log tab */
		TSharedPtr<SConcertTransportLog> GlobalTransportLog;
		
		void CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FArguments& InArgs);
		TSharedRef<SDockTab> SpawnClientBrowserTab(const FSpawnTabArgs& InTabArgs);
		TSharedRef<SDockTab> SpawnGlobalLogTab(const FSpawnTabArgs& InTabArgs);

		TSharedRef<SWidget> CreateOpenGlobalLogButton() const;
	};
}
