// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPromptConcertLoggingEnabled.h"

#include "ConcertTransportEvents.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"

void SPromptConcertLoggingEnabled::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f, 0.8f)))
		.Padding(0)
		[
			SNew(SBox)
			.Padding(0.f, 0.f, 0.f, 10.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EnableLoggingVisibility", "Execute Concert.EnableLogging to use this view."))
					.Justification(ETextJustify::Center)
				]

				+SVerticalBox::Slot()
				.Padding(0, 4, 0, 0)
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SButton)
					.OnClicked_Lambda([](){ ConcertTransportEvents::SetLoggingEnabled(true); return FReply::Handled(); })
					.ToolTipText(LOCTEXT("EnableLoggingVisibility.Button.Tooltip", "Executes Concert.EnableLogging"))
					[
						SNew(STextBlock)
							.Text(LOCTEXT("EnableLoggingVisibility.Button.Text", "Enable logging"))
					]
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
