// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

/** Placed in an overlay, reminds the user to execute Concert.EnableLogging if logging is not enabled. */
class SPromptConcertLoggingEnabled : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPromptConcertLoggingEnabled) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
