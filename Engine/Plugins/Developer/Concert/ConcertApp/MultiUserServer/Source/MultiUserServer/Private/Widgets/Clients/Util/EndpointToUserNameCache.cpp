// Copyright Epic Games, Inc. All Rights Reserved.

#include "EndpointToUserNameCache.h"

#include "ConcertMessages.h"
#include "ConcertServerEvents.h"
#include "ConcertUtil.h"
#include "ConcertServerUtil.h"
#include "IConcertServer.h"
#include "IConcertSession.h"
#include "INetworkMessagingExtension.h"
#include "Algo/AnyOf.h"
#include "Features/IModularFeatures.h"

FEndpointToUserNameCache::FEndpointToUserNameCache(TSharedRef<IConcertServer> InServer)
	: Server(MoveTemp(InServer))
{
	ConcertServerEvents::OnLiveSessionCreated().AddRaw(this, &FEndpointToUserNameCache::OnLiveSessionCreated);
	ConcertServerEvents::OnLiveSessionDestroyed().AddRaw(this, &FEndpointToUserNameCache::OnLiveSessionDestroyed);

	for (TSharedPtr<IConcertServerSession> LiveSession : Server->GetLiveSessions())
	{
		RegisterLiveSession(LiveSession.ToSharedRef());
		for (const FConcertSessionClientInfo& ClientInfo : LiveSession->GetSessionClients())
		{
			CacheClientInfo(*LiveSession, ClientInfo);
		}
	}

	Server->OnRemoteEndpointConnectionChanged().AddRaw(this, &FEndpointToUserNameCache::OnAdminEndpointConnectionChanged);
}

FEndpointToUserNameCache::~FEndpointToUserNameCache()
{
	ConcertServerEvents::OnLiveSessionCreated().RemoveAll(this);
	ConcertServerEvents::OnLiveSessionDestroyed().RemoveAll(this);

	for (TWeakPtr<IConcertServerSession> Session : SubscribedToSessions)
	{
		if (TSharedPtr<IConcertServerSession> PinnedSession = Session.Pin())
		{
			PinnedSession->OnSessionClientChanged().RemoveAll(this);
		}
	}
}

bool FEndpointToUserNameCache::IsServerEndpoint(const FGuid& EndpointId) const
{
	const bool bIsLocalSessionEndpoint = Algo::AnyOf(Server->GetLiveSessions(), [&EndpointId](const TSharedPtr<IConcertServerSession>& Session)
	{
		return Session->GetSessionInfo().ServerEndpointId == EndpointId;
	});
	return bIsLocalSessionEndpoint || Server->GetServerInfo().AdminEndpointId == EndpointId;
}

TOptional<FConcertClientInfo> FEndpointToUserNameCache::GetClientInfo(const FGuid& EndpointId) const
{
	// Already known or an old endpoint ID that is no longer valid (e.g. because client left session)
	if (const FNodeEndpointId* CachedNodeId = CachedConcertEndpointToNodeEndpoints.Find(EndpointId))
	{
		if (const FConcertClientInfo* ClientData = CachedClientData.Find(*CachedNodeId))
		{
			return *ClientData;
		}
	}
	
	auto LookUpAddress = [this](const FMessageAddress& MessageAddress) -> TOptional<FConcertClientInfo>
	{
		const FNodeEndpointId NodeEndpointId = GetNodeIdFromMessagingBackend(MessageAddress);
		if (const FConcertClientInfo* Data = CachedClientData.Find(NodeEndpointId)) 
		{
			return *Data;
		}
		return {};
	};

	// Is in any live session?
	if (const TSharedPtr<IConcertServerSession> Session = ConcertUtil::GetLiveSessionClientConnectedTo(Server.Get(), EndpointId))
	{
		const FMessageAddress ConnectedClientAddress = Session->GetClientAddress(EndpointId);
		return LookUpAddress(ConnectedClientAddress);
	}

	// Is somebody trying to discover sessions? Data will only be available after this client has joined at least one session
	if (const FMessageAddress RemoteAdminAddress = Server->GetRemoteAddress(EndpointId); RemoteAdminAddress.IsValid())
	{
		return LookUpAddress(RemoteAdminAddress);
	}
	return {};
}

TOptional<FConcertClientInfo> FEndpointToUserNameCache::GetClientInfoFromNodeId(const FNodeEndpointId& EndpointId) const
{
	if (const FConcertClientInfo* ClientData = CachedClientData.Find(EndpointId))
	{
		return *ClientData;
	}
	return {};
}

TOptional<FEndpointToUserNameCache::FNodeEndpointId> FEndpointToUserNameCache::TranslateEndpointIdToNodeId(const FGuid& EndpointId) const
{
	const FNodeEndpointId* CachedNodeId = CachedConcertEndpointToNodeEndpoints.Find(EndpointId);
	if (CachedNodeId)
	{
		return *CachedNodeId;
	}
	return {};
}

FString FEndpointToUserNameCache::GetEndpointDisplayString(const FGuid& EndpointId) const
{
	if (IsServerEndpoint(EndpointId))
	{
		return FString("Server");		
	}

	if (const TOptional<FConcertClientInfo> ClientInfo = GetClientInfo(EndpointId))
	{
		return ClientInfo->DisplayName;
	}
	
	return EndpointId.ToString(EGuidFormats::DigitsWithHyphens);
}

void FEndpointToUserNameCache::OnLiveSessionCreated(bool bSuccess, const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession)
{
	RegisterLiveSession(InLiveSession);
}

void FEndpointToUserNameCache::OnLiveSessionDestroyed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession)
{
	// No need to unsubscribe because the session will be destroyed anyways
	SubscribedToSessions.Remove(InLiveSession);
}

void FEndpointToUserNameCache::OnClientInfoChanged(IConcertServerSession& Session, EConcertClientStatus ConnectionStatus, const FConcertSessionClientInfo& ClientInfo)
{
	switch (ConnectionStatus)
	{
	case EConcertClientStatus::Disconnected:
		break;
		
	case EConcertClientStatus::Connected:
	case EConcertClientStatus::Updated:
		CacheClientInfo(Session, ClientInfo);
		break;
	default:
		checkNoEntry();
	}
}

void FEndpointToUserNameCache::OnAdminEndpointConnectionChanged(const FConcertEndpointContext& ConcertEndpointContext, EConcertRemoteEndpointConnection ConcertRemoteEndpointConnection)
{
	const FMessageAddress Address = Server->GetRemoteAddress(ConcertEndpointContext.EndpointId);
	if (const FNodeEndpointId NodeEndpointId = GetNodeIdFromMessagingBackend(Address); ConcertRemoteEndpointConnection == EConcertRemoteEndpointConnection::Discovered && NodeEndpointId.IsValid())
	{
		CachedConcertEndpointToNodeEndpoints.Add(ConcertEndpointContext.EndpointId, NodeEndpointId);
	}
}

void FEndpointToUserNameCache::RegisterLiveSession(const TSharedRef<IConcertServerSession>& InLiveSession)
{
	SubscribedToSessions.Add(InLiveSession);
	InLiveSession->OnSessionClientChanged().AddRaw(this, &FEndpointToUserNameCache::OnClientInfoChanged);
}

void FEndpointToUserNameCache::CacheClientInfo(const IConcertServerSession& Session, const FConcertSessionClientInfo& ClientInfo)
{
	const FMessageAddress Address = Session.GetClientAddress(ClientInfo.ClientEndpointId);
	if (const FNodeEndpointId NodeEndpointId = GetNodeIdFromMessagingBackend(Address); NodeEndpointId.IsValid())
	{
		CachedConcertEndpointToNodeEndpoints.Add(ClientInfo.ClientEndpointId, NodeEndpointId);
		CachedClientData.Add(NodeEndpointId, ClientInfo.ClientInfo);
	}
}

FEndpointToUserNameCache::FNodeEndpointId FEndpointToUserNameCache::GetNodeIdFromMessagingBackend(const FMessageAddress& MessageAddress) const
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const INetworkMessagingExtension* Extension = ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName)
		? &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName)
		: nullptr;
	if (Extension)
	{
		return Extension->GetNodeIdFromAddress(MessageAddress);
	}
	return {};
}
