// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"

enum class EConcertRemoteEndpointConnection : uint8;
struct FConcertEndpointContext;
struct FMessageAddress;
template <typename OptionalType> struct TOptional;

class IConcertServerSession;
class IConcertServer;

enum class EConcertClientStatus : uint8;

struct FConcertSessionClientInfo;

/**
 * Responsible for keeping hold of client info even when the client disconnects.
 * Client info is removed server-side upon disconnect but logs need their display info.
 */
class FEndpointToUserNameCache
{
public:
	
	/** ID used by the messaging system - corresponds to an IP address. */
	using FNodeEndpointId = FGuid;

	FEndpointToUserNameCache(TSharedRef<IConcertServer> Server);
	~FEndpointToUserNameCache();

	bool IsServerEndpoint(const FGuid& EndpointId) const;
	/** Gets the client info given from any endpoint ID (any ID of a FConcertRemoteEndpoint) */
	TOptional<FConcertClientInfo> GetClientInfo(const FGuid& EndpointId) const;
	/** Gets the client info given from any endpoint node ID (the ID from the messaging system node - corresponds to IP address). */
	TOptional<FConcertClientInfo> GetClientInfoFromNodeId(const FNodeEndpointId& EndpointId) const;

	/** Translates a Concert endpoint Id to the corresponding messaging node Id */
	TOptional<FNodeEndpointId> TranslateEndpointIdToNodeId(const FGuid& EndpointId) const;

	/** Gets either the client's name or "Server" depending on type of endpoint this is */
	FString GetEndpointDisplayString(const FGuid& EndpointId) const;

private:

	/** Used to unsubscribe when we're destroyed */
	TSharedRef<IConcertServer> Server;
	/** Used to unsubscribe when we're destroyed */
	TSet<TWeakPtr<IConcertServerSession>> SubscribedToSessions;

	/**
	 * The client info we're caching.
	 * 
	 * Concert may generate multiple endpoint IDs for a single remote machine.
	 * However, the remote node ID is always unique and is retrieved by querying the UDP backend (see GetNodeIdFromMessagingBackend).
	 */
	TMap<FNodeEndpointId, FConcertClientInfo> CachedClientData;
	/**
	 * Keeps track of past endpoints IDs that may now no longer be valid.
	 * Every time a client joins a session, a new endpoint ID is generated for that client. It becomes impossible to
	 * lookup old concert endpoint IDs without this mapping
	 */
	TMap<FGuid, FNodeEndpointId> CachedConcertEndpointToNodeEndpoints;

	void OnLiveSessionCreated(bool bSuccess, const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession);
	void OnLiveSessionDestroyed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession);
	void OnClientInfoChanged(IConcertServerSession& Session, EConcertClientStatus ConnectionStatus, const FConcertSessionClientInfo& ClientInfo);
	
	void OnAdminEndpointConnectionChanged(const FConcertEndpointContext& ConcertEndpointContext, EConcertRemoteEndpointConnection ConcertRemoteEndpointConnection);

	void RegisterLiveSession(const TSharedRef<IConcertServerSession>& InLiveSession);
	void CacheClientInfo(const IConcertServerSession& Session, const FConcertSessionClientInfo& ClientInfo);
	
	FNodeEndpointId GetNodeIdFromMessagingBackend(const FMessageAddress& MessageAddress) const;
};
