// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActivityDependencyDialog.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "MultiUserServerConsoleVariables.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "MultiUserServerModule.h"

#include "ConcertLogGlobal.h"
#include "Dialog/SMessageDialog.h"
#include "HistoryEdition/ActivityNode.h"
#include "HistoryEdition/DebugDependencyGraph.h"
#include "HistoryEdition/DependencyGraphBuilder.h"
#include "Templates/NonNullPointer.h"
#include "Widgets/HistoryEdition/SActivityDependencyView.h"
#include "Window/ModalWindowManager.h"
#include "Widgets/SessionTabs/Archived/ArchivedSessionHistoryController.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SActivityDependencyDialog"

namespace UE::MultiUserServer
{
	void SActivityDependencyDialog::Construct(const FArguments& InArgs, const FGuid& SessionId, const TSharedRef<IConcertSyncServer>& SyncServer, TSet<FActivityID> InBaseActivities)
	{
		check(InArgs._OnConfirmAction.IsBound() && InArgs._AnalyseActivities.IsBound());
		OnConfirmActionFunc = InArgs._OnConfirmAction;
		GetFinalInclusionResultTextFunc = InArgs._GetFinalInclusionResultText;
		ConfirmDialogTitle = InArgs._ConfirmDialogTitle;
		ConfirmDialogMessage = InArgs._ConfirmDialogMessage;
		
		SCustomDialog::Construct(
			SCustomDialog::FArguments()
			.AutoCloseOnButtonPress(false)
			.Title(InArgs._Title)
			.Buttons({
				FButton(InArgs._PerformActionButtonLabel)
					.SetOnClicked(FSimpleDelegate::CreateSP(this, &SActivityDependencyDialog::OnConfirmPressed)),
				FButton(LOCTEXT("DeleteActivity.CancelButtonLabel", "Cancel"))
					.SetOnClicked(FSimpleDelegate::CreateLambda([this]()
					{
						RequestDestroyWindow();
					}))
					.SetPrimary(true)
			})
			.Content()
			[
				SNew(SVerticalBox)
				
				+SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.AutoHeight()
				.Padding(0, 5, 0, 10)
				[
					SNew(STextBlock)
					.Text(InArgs._Description)
				]

				+SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					CreateBody(InArgs, SessionId, SyncServer, MoveTemp(InBaseActivities))
				]
			]
		);
	}

	TSharedRef<SWidget> SActivityDependencyDialog::CreateBody(const FArguments& InArgs, const FGuid& InSessionId, const TSharedRef<IConcertSyncServer>& InSyncServer, TSet<FActivityID> InBaseActivities)
	{
		const TOptional<FConcertSyncSessionDatabaseNonNullPtr> SessionDatabase = InSyncServer->GetArchivedSessionDatabase(InSessionId);
		check(SessionDatabase);
		
		ConcertSyncCore::FActivityDependencyGraph DependencyGraph = ConcertSyncCore::BuildDependencyGraphFrom(**SessionDatabase);
		if (ConsoleVariables::CVarLogActivityDependencyGraphOnDelete.GetValueOnGameThread())
		{
			UE_LOGF(LogConcert, Log, "Analysed graph: %ls", *UE::ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, **SessionDatabase));
		}
		
		return SAssignNew(DependencyView, SActivityDependencyView, MoveTemp(DependencyGraph))
			.AdditionalColumns(InArgs._AdditionalColumns)
			.BaseActivities(MoveTemp(InBaseActivities))
			.AnalyseActivities(InArgs._AnalyseActivities)
			.ShouldShowActivity(InArgs._ShouldShowActivity)
			.CreateSessionHistoryController_Lambda([this, InSessionId, InSyncServer](const SSessionHistory::FArguments& InArgs)
			{
				const TSharedPtr<FArchivedSessionHistoryController> FilteredSessionHistoryController = CreateForDeletionDialog(
					InSessionId,
					InSyncServer,
					InArgs
					);
				return FilteredSessionHistoryController.ToSharedRef();
			})
			.GetCheckboxToolTip_Lambda([this](const int64 ActivityId, bool bIsHardDependency)
			{
				const bool bIsChecked = DependencyView->IsChecked(ActivityId);
				if (bIsHardDependency)
				{
					return FText::Format(LOCTEXT("Action.CheckBox.TooltipDisabledFmt", "Hard dependency. {0}"), GetFinalInclusionResultTextFunc.Execute(ActivityId, bIsChecked));
				}

				if (bIsChecked)
				{
					return FText::Format(LOCTEXT("Action.CheckBox.TooltipEnabled.ApplyActionFmt", "Possible dependency.\n{0}"), GetFinalInclusionResultTextFunc.Execute(ActivityId, bIsChecked));
				}
				return FText::Format(LOCTEXT("Action.CheckBox.TooltipEnabled.DoNotApplyActionFmt", "Possible dependency.\n{0}"), GetFinalInclusionResultTextFunc.Execute(ActivityId, bIsChecked));
			});
	}

	void SActivityDependencyDialog::OnConfirmPressed()
	{
		const TSharedRef<SMessageDialog> Dialog = SNew(SMessageDialog)
			.Title(ConfirmDialogTitle.IsSet() ? ConfirmDialogTitle.Get() : LOCTEXT("ConfirmDialog.Title", "Confirm action"))
			.Buttons({
				FButton(LOCTEXT("ConfirmDialog.Yes", "Yes"))
					.SetOnClicked(FSimpleDelegate::CreateLambda([KeepAlive = SharedThis(this)]()
					{
						KeepAlive->OnConfirmActionFunc.Execute(KeepAlive->DependencyView->GetSelection());
						KeepAlive->RequestDestroyWindow();
					})),
				FButton(LOCTEXT("DeleteActivity.CancelButtonLabel", "Cancel"))
					.SetPrimary(true)
			})
			.Message(ConfirmDialogMessage.IsSet() ? ConfirmDialogMessage.Get() : LOCTEXT("ConfirmDialog.DefaultMessage", "Are you sure?"));
		FConcertServerUIModule::Get()
			.GetModalWindowManager()
			->ShowFakeModalWindow(Dialog);
	}
}

#undef LOCTEXT_NAMESPACE
