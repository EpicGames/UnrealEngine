// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dialog/SCustomDialog.h"

#include "SActivityDependencyView.h"

class FArchivedSessionHistoryController;
class IConcertSyncServer;

namespace UE::ConcertSyncCore
{
	struct FHistoryAnalysisResult;
}

namespace UE::MultiUserServer
{
	/** Base dialog for perform actions activity dependencies, notably for deleting and muting activities. */
	class SActivityDependencyDialog : public SCustomDialog
	{
	public:

		DECLARE_DELEGATE_OneParam(FConfirmDependencyAction, const TSet<FActivityID>& /*SelectedItems*/)
		DECLARE_DELEGATE_RetVal_TwoParams(FText, FGetFinalInclusionResultText, const int64 /*ActivityId*/, bool /*bIsChecked*/);

		SLATE_BEGIN_ARGS(SActivityDependencyDialog)
		{}
			SLATE_ARGUMENT(FText, Title)
			SLATE_ARGUMENT(FText, Description)
			SLATE_ARGUMENT(FText, PerformActionButtonLabel)
		
			/** Called when the user confirms the deletion of the activities.*/
			SLATE_EVENT(FConfirmDependencyAction, OnConfirmAction)

			/** Analyses the given activities */
			SLATE_EVENT(SActivityDependencyView::FAnalyseActivities, AnalyseActivities)
		
			/** Appended to tooltip when hovering the checkbox in the first column. Explains what the current check state means. */
			SLATE_EVENT(FGetFinalInclusionResultText, GetFinalInclusionResultText)

			/** Optional. Whether to show the activity. */
			SLATE_EVENT(SActivityDependencyView::FShouldShowActivity, ShouldShowActivity)
		
			/** Optional. Additional columns to add on top of the default ones. */
			SLATE_ARGUMENT(TArray<FActivityColumn>, AdditionalColumns)
		
			/** Optional. The title of the dialog that shows up when the user clicks the confirm button. */
			SLATE_ATTRIBUTE(FText, ConfirmDialogTitle)
			/** Optional. The content of the dialog that shows up when the user clicks the confirm button. */
			SLATE_ATTRIBUTE(FText, ConfirmDialogMessage)
		SLATE_END_ARGS()

		/**
		 * @param InDependencyArgs Specifies which activities must be deleted and which are optional.
		 */
		void Construct(const FArguments& InArgs, const FGuid& SessionId, const TSharedRef<IConcertSyncServer>& SyncServer, TSet<FActivityID> InBaseActivities);
		
	private:

		TSharedPtr<SActivityDependencyView> DependencyView;
		
		FConfirmDependencyAction OnConfirmActionFunc;
		FGetFinalInclusionResultText GetFinalInclusionResultTextFunc;
		
		TAttribute<FText> ConfirmDialogTitle;
		TAttribute<FText> ConfirmDialogMessage;

		TSharedRef<SWidget> CreateBody(const FArguments& InArgs, const FGuid& InSessionId, const TSharedRef<IConcertSyncServer>& InSyncServer, TSet<FActivityID> InBaseActivities);

		void OnConfirmPressed();
	};
}

