// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActivityDependencyView.h"

#include "ConcertServerStyle.h"
#include "Algo/AllOf.h"
#include "ConcertSyncSessionTypes.h"
#include "Session/History/AbstractSessionHistoryController.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "Widgets/Input/SButton.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "Widgets/Input/SCheckBox.h"
#include "HistoryEdition/ActivityNode.h"
#include "Widgets/Layout/SScaleBox.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SActivityDependencyView"

namespace UE::MultiUserServer
{
	namespace DependencyView
	{
		const FName ShouldPerformActivityActionCheckboxColumnID("ShouldPerformActivityActionCheckboxColumnID");
		DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FCreateActivityColumnWidget, FActivityID /*ActivityID*/);
		
		static FActivityColumn ShouldPerformActivityActionColumn(const FCreateActivityColumnWidget& CreateActivityColumnWidget)
		{
			return FActivityColumn(
				SHeaderRow::Column(ShouldPerformActivityActionCheckboxColumnID)
					.DefaultLabel(FText::GetEmpty())
					.FixedWidth(22)
					.ShouldGenerateWidget(true)
				)
				.ColumnSortOrder(static_cast<int32>(ConcertSharedSlate::ActivityColumn::EPredefinedColumnOrder::AvatarColor))
				.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([CreateActivityColumnWidget](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
				{
					Slot
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						CreateActivityColumnWidget.Execute(Activity->Activity.ActivityId)
					];
				}));
		}
	}
	
	void SActivityDependencyView::Construct(const FArguments& InArgs, ConcertSyncCore::FActivityDependencyGraph InDependencyGraph)
	{
		using namespace DependencyView;
		check(InArgs._CreateSessionHistoryController.IsBound() && InArgs._AnalyseActivities.IsBound());

		BaseActivities = InArgs._BaseActivities;
		AnalyseActivitiesDelegate = InArgs._AnalyseActivities;
		ShouldShowActivity = InArgs._ShouldShowActivity;
		GetCheckboxToolTipDelegate = InArgs._GetCheckboxToolTip;

		DependencyGraph = MoveTemp(InDependencyGraph);
		UserSelectedActivities = BaseActivities;
		Selection = AnalyseActivitiesDelegate.Execute(DependencyGraph, BaseActivities);
		
		TSharedPtr<FConcertSessionActivitiesOptions> ViewOptions = MakeShared<FConcertSessionActivitiesOptions>();
		ViewOptions->bEnableConnectionActivityFiltering = false;
		ViewOptions->bEnableLockActivityFiltering = false;
		ViewOptions->bEnableIgnoredActivityFiltering = false;

		const FActivityColumn CheckmarkColumn = ShouldPerformActivityActionColumn(FCreateActivityColumnWidget::CreateLambda([this](FActivityID ActivityID)
		{
			const bool bIsHardDependency = Selection.HardDependencies.Contains(ActivityID);
			return SNew(SCheckBox)
				.IsEnabled_Lambda([this, ActivityID]()
				{
					return !BaseActivities.Contains(ActivityID)
						&& (UserSelectedActivities.Contains(ActivityID) || Selection.PossibleDependencies.Contains(ActivityID));
				})
				.IsChecked_Lambda([this, ActivityID]()
				{
					return IsChecked(ActivityID) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this, ActivityID](ECheckBoxState NewState)
				{
					OnCheckBoxStateChanged(ActivityID, NewState == ECheckBoxState::Checked);
				})
				.ToolTipText_Lambda([this, ActivityID, bIsHardDependency]()
				{
					return GetCheckboxToolTipDelegate.IsBound() ? GetCheckboxToolTipDelegate.Execute(ActivityID, bIsHardDependency) : FText::GetEmpty();
				});
		}));
		
		TArray<FActivityColumn> Columns { CheckmarkColumn, ConcertSharedSlate::ActivityColumn::Operation() };
		for (const FActivityColumn& Column : InArgs._AdditionalColumns)
		{
			Columns.Add(Column);
		}

		SessionHistoryController = InArgs._CreateSessionHistoryController.Execute(
			SSessionHistory::FArguments()
				.AllowActivity_Lambda([this](const FConcertSyncActivity& Activity, const TStructOnScope<FConcertSyncActivitySummary>&)
				{
					const bool bShouldShow = !ShouldShowActivity.IsBound() || ShouldShowActivity.Execute(Activity);
					return bShouldShow &&
						(Selection.HardDependencies.Contains(Activity.ActivityId)
						|| Selection.PossibleDependencies.Contains(Activity.ActivityId));
				})
				.Columns(Columns)
				.ViewOptions(ViewOptions)
				.DetailsAreaVisibility(EVisibility::Collapsed)
				.DarkenMutedActivities(false)
				.OnMakeColumnOverlayWidget(this, &SActivityDependencyView::CreateCheckboxOverlay)
			);
		ChildSlot
		[
			SessionHistoryController->GetSessionHistory()
		];
	}

	bool SActivityDependencyView::IsChecked(FActivityID ActivityID) const
	{
		return UserSelectedActivities.Contains(ActivityID) || Selection.HardDependencies.Contains(ActivityID);
	}

	TSet<FActivityID> SActivityDependencyView::GetSelection() const
	{
		TSet<FActivityID> Result = UserSelectedActivities;
		for (const FActivityID HardDependency : Selection.HardDependencies)
		{
			Result.Add(HardDependency);
		}
		return Result;
	}

	void SActivityDependencyView::OnCheckBoxStateChanged(const FActivityID ActivityId, bool bIsChecked)
	{
		check(!BaseActivities.Contains(ActivityId));
		if (bIsChecked)
		{
			UserSelectedActivities.Add(ActivityId);
		}
		else
		{
			UserSelectedActivities.Remove(ActivityId);
		}

		Selection = AnalyseActivitiesDelegate.Execute(DependencyGraph, UserSelectedActivities);
		SessionHistoryController->ReloadActivities();
	}

	TSharedPtr<SWidget> SActivityDependencyView::CreateCheckboxOverlay(TWeakPtr<SMultiColumnTableRow<TSharedPtr<FConcertSessionActivity>>> Row, TWeakPtr<FConcertSessionActivity> Activity, const FName& ColumnName)
	{
		TSharedPtr<FConcertSessionActivity> ActivityPin = Activity.Pin();
		check(ActivityPin);

		const FActivityID ActivityID = ActivityPin->Activity.ActivityId;
		const bool bCanEverCreate = !BaseActivities.Contains(ActivityID);
		if (ColumnName != DependencyView::ShouldPerformActivityActionCheckboxColumnID || !bCanEverCreate)
		{
			return nullptr;
		}

		return SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
			.ForegroundColor(FSlateColor::UseStyle())
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Visibility_Lambda([this, ActivityID, Row]()
			{
				return Row.Pin()->IsHovered() && CanRemoveMandatorySelection(ActivityID) ? EVisibility::Visible : EVisibility::Hidden;
			})
			.ToolTipText(LOCTEXT("MandatorilyChecked", "Click to highlight the activities that mandate this activity's selection."))
			.OnClicked_Lambda([this, ActivityID]()
			{
				const TOptional<ConcertSyncCore::FActivityNodeID> NodeId = DependencyGraph.FindNodeByActivity(ActivityID);
				if (ensure(NodeId))
				{
					const ConcertSyncCore::FActivityNode& Node = DependencyGraph.GetNodeById(*NodeId);
					const TArray<TSharedPtr<FConcertSessionActivity>>& Activities = SessionHistoryController->GetSessionHistory()->GetActivities();
					TArray<TSharedPtr<FConcertSessionActivity>> NewItemSelection;
					for (const TSharedPtr<FConcertSessionActivity>& Activity : Activities)
					{
						if (Node.DependsOnActivity(Activity->Activity.ActivityId, DependencyGraph, {}, ConcertSyncCore::EDependencyStrength::HardDependency))
						{
							NewItemSelection.Add(Activity);
						}
					}
					SessionHistoryController->GetSessionHistory()->SetSelectedActivities(NewItemSelection);
				}
				return FReply::Handled();
			})
			.ContentPadding(0)
			[
				SNew(SScaleBox)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FConcertServerStyle::Get().GetBrush("Concert.HighlightHardDependencies"))
				]
			];
	}

	bool SActivityDependencyView::CanRemoveMandatorySelection(const FActivityID ActivityID) const
	{
		const TOptional<ConcertSyncCore::FActivityNodeID> NodeId = DependencyGraph.FindNodeByActivity(ActivityID);
		const bool bCanRemoveMandatorySelection = ensure(NodeId)
			&& Algo::AllOf(DependencyGraph.GetNodeById(*NodeId).GetDependencies(), [this](const ConcertSyncCore::FActivityDependencyEdge& DependencyEdge)
			{
				const FActivityID TargetActivityId = DependencyGraph.GetNodeById(DependencyEdge.GetDependedOnNodeID()).GetActivityId();
				return !BaseActivities.Contains(TargetActivityId);
			});
		return IsChecked(ActivityID) && !UserSelectedActivities.Contains(ActivityID) && bCanRemoveMandatorySelection;
	}
}

#undef LOCTEXT_NAMESPACE 
