// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HistoryEdition/ActivityDependencyGraph.h"
#include "HistoryEdition/HistoryAnalysis.h"
#include "Session/History/SSessionHistory.h"

class FAbstractSessionHistoryController;
template<typename T> class SMultiColumnTableRow;
struct FConcertSyncActivity;

namespace UE::MultiUserServer
{
	/**
	 * A view for displaying activity dependencies.
	 * Each row has a checkbox. Useful for actions, such as deleting, muting, and unmuting.
	 * Adds additional dependencies depending on what optional dependencies are checked.
	 */
	class SActivityDependencyView : public SCompoundWidget
	{
	public:

		DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<FAbstractSessionHistoryController>, FCreateSessionHistoryController, const SSessionHistory::FArguments& /*InArgs*/)
		DECLARE_DELEGATE_RetVal_TwoParams(ConcertSyncCore::FHistoryAnalysisResult, FAnalyseActivities, const ConcertSyncCore::FActivityDependencyGraph&, const TSet<FActivityID>& /*Activities*/);
		DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldShowActivity, const FConcertSyncActivity /*ActivityId*/);
		DECLARE_DELEGATE_RetVal_TwoParams(FText, FGetCheckboxToolTip, const int64& /*Activity*/, bool /*bIsHardDependency*/)

		SLATE_BEGIN_ARGS(SActivityDependencyView)
		{}
			/** The activities this view is based on. Always checked. */
			SLATE_ARGUMENT(TSet<FActivityID>, BaseActivities)
		
			/** Required. A function that will create a session history widget. */
			SLATE_EVENT(FCreateSessionHistoryController, CreateSessionHistoryController)
			/** Analyses the given activities */
			SLATE_EVENT(FAnalyseActivities, AnalyseActivities)

			/** Optional. Whether to show the activity */
			SLATE_EVENT(FShouldShowActivity, ShouldShowActivity)
			/** Optional. The tooltip to display when the user hovers a */
			SLATE_EVENT(FGetCheckboxToolTip, GetCheckboxToolTip)
		
			/** Optional. Additional columns to add on top of the default ones. */
			SLATE_ARGUMENT(TArray<FActivityColumn>, AdditionalColumns)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, ConcertSyncCore::FActivityDependencyGraph InDependencyGraph);

		/** @return Whether the given activity ID is checked */
		bool IsChecked(FActivityID ActivityID) const;
		TSet<FActivityID> GetSelection() const;
		
	private:

		/** Used to render the rows */
		TSharedPtr<FAbstractSessionHistoryController> SessionHistoryController;
		
		/** Built once on construction. */
		ConcertSyncCore::FActivityDependencyGraph DependencyGraph;
		
		/** The activities this view is based on. Always checked. */
		TSet<FActivityID> BaseActivities;
		TSet<FActivityID> UserSelectedActivities;

		/** The current selection. Updated every time something is checked. */
		ConcertSyncCore::FHistoryAnalysisResult Selection;

		FAnalyseActivities AnalyseActivitiesDelegate;
		FShouldShowActivity ShouldShowActivity;
		FGetCheckboxToolTip GetCheckboxToolTipDelegate;

		void OnCheckBoxStateChanged(const FActivityID ActivityId, bool bIsChecked);
		TSharedPtr<SWidget> CreateCheckboxOverlay(TWeakPtr<SMultiColumnTableRow<TSharedPtr<FConcertSessionActivity>>> Row, TWeakPtr<FConcertSessionActivity> Activity, const FName& ColumnName);

		bool CanRemoveMandatorySelection(const FActivityID ActivityID) const;
	};
}
