// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeleteActivityDependenciesDialog.h"
#include "Widgets/HistoryEdition/SActivityDependencyDialog.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SDeleteActivityDependenciesDialog"

void UE::MultiUserServer::SDeleteActivityDependenciesDialog::Construct(const FArguments& InArgs, const FGuid& SessionId, const TSharedRef<IConcertSyncServer>& SyncServer, TSet<FActivityID> InBaseActivities)
{
	SActivityDependencyDialog::Construct(
		SActivityDependencyDialog::FArguments()
			.Title(LOCTEXT("RemoveActivityTitle", "Remove activity"))
			.Description(LOCTEXT("DeleteActivity.HeaderBaseText", "Review the activities that will be deleted:"))
			.PerformActionButtonLabel(LOCTEXT("DeleteActivity.ConfirmButtonLabel", "Delete"))
			.OnConfirmAction(InArgs._OnConfirmDeletion)
			.AnalyseActivities_Lambda([](const ConcertSyncCore::FActivityDependencyGraph& Graph, const TSet<FActivityID>& ActivityIds)
			{
				return ConcertSyncCore::AnalyseActivityDependencies_TopDown(ActivityIds, Graph, true);
			})
			.GetFinalInclusionResultText_Lambda([](const int64 ActivityId, bool bIsChecked)
			{
				return bIsChecked
					? LOCTEXT("WillDelete", "Will delete.")
					: LOCTEXT("WillNotDelete", "Won't delete.");
			})
			.ConfirmDialogTitle(LOCTEXT("ConfirmDeletion.Title", "Confirm deletion"))
			.ConfirmDialogMessage(LOCTEXT("ConfirmDeletion.Description", "Are you sure you want to delete the selected activities?")),
		SessionId, SyncServer, MoveTemp(InBaseActivities)
	);
}
#undef LOCTEXT_NAMESPACE
