// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SActivityDependencyDialog.h"

class FArchivedSessionHistoryController;
namespace UE::ConcertSyncCore
{
	struct FHistoryAnalysisResult;
}

namespace UE::MultiUserServer
{
	/** Displayed when a user asks to delete an activity. */
	class SDeleteActivityDependenciesDialog : public SActivityDependencyDialog
	{
	public:
		SLATE_BEGIN_ARGS(SDeleteActivityDependenciesDialog)
		{}
			/** Called when the user confirms the deletion of the activities.*/
			SLATE_EVENT(SActivityDependencyDialog::FConfirmDependencyAction, OnConfirmDeletion)
		SLATE_END_ARGS()

		/**
		 * @param InDeletionRequirements Specifies which activities must be deleted and which are optional.
		 */
		void Construct(const FArguments& InArgs, const FGuid& SessionId, const TSharedRef<IConcertSyncServer>& SyncServer, TSet<FActivityID> InBaseActivities);
	};
}


