// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMuteActivityDependenciesDialog.h"

#include "ConcertServerStyle.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"
#include "IConcertSyncServer.h"
#include "Templates/NonNullPointer.h"
#include "Widgets/HistoryEdition/SActivityDependencyDialog.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SMuteActivityDependenciesDialog"

namespace UE::MultiUserServer
{
	namespace MuteActivityDependenciesDialog
	{
		const FName IsMutedColumnID("IsMutedColumn");

		DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FCreateActivityColumnWidget, FActivityID /*ActivityID*/);
	
		static FActivityColumn IsMutedColumn(const FCreateActivityColumnWidget& CreateActivityColumnWidget)
		{
			return FActivityColumn(
				SHeaderRow::Column(IsMutedColumnID)
					.DefaultLabel(FText::GetEmpty())
					.FixedWidth(20)
					.ShouldGenerateWidget(true)
				)
				// Place it right after the checkbox, see ShouldPerformActivityActionColumn in SActivityDependencyDialog.cpp
				.ColumnSortOrder(static_cast<int32>(ConcertSharedSlate::ActivityColumn::EPredefinedColumnOrder::AvatarColor) + 1)
				.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([CreateActivityColumnWidget](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
				{
					Slot
					.VAlign(VAlign_Center)
					[
						CreateActivityColumnWidget.Execute(Activity->Activity.ActivityId)
					];
				}));
		}
	}

	void SMuteActivityDependenciesDialog::Construct(const FArguments& InArgs, const FGuid& InSessionId, const TSharedRef<IConcertSyncServer>& InSyncServer, TSet<FActivityID> InBaseActivities)
	{
		SyncServer = InSyncServer;
		SessionId = InSessionId;
		
		using namespace MuteActivityDependenciesDialog;
		const FActivityColumn MutedColumn = IsMutedColumn(FCreateActivityColumnWidget::CreateLambda([this](FActivityID ActivityID)
		{
			const bool bIsMuted = IsMuted(ActivityID);
			return SNew(SImage)
				.Image(bIsMuted ? FConcertServerStyle::Get().GetBrush("Concert.Muted") : FConcertServerStyle::Get().GetBrush("Concert.Unmuted"))
				.ToolTipText(bIsMuted ? LOCTEXT("Muted", "Muted. Server does not send this activity to clients.") : LOCTEXT("Unmuted", "Not muted. Server sends this activity to clients."));
		}));

		const bool bShouldMute = InArgs._MuteOperation == EMuteOperation::Mute;
		const FText DialogTitle = bShouldMute
			? LOCTEXT("Mute.Title", "Mute activities")
			: LOCTEXT("Unmute.Title", "Unmute activities");
		const FText Description = bShouldMute
			? LOCTEXT("Mute.Description", "Review the activities that will be muted.\nMuted activities are not sent to clients.")
			: LOCTEXT("Unmute.Description", "Review the activities that will be unmuted.\nMuted activities are not sent to clients.");
		const FText ButtonLabel = bShouldMute
			? LOCTEXT("Mute.ButtonLabel", "Mute")
			: LOCTEXT("Unmute.ButtonLabel", "Unmute");
		const FText ConfirmDialogTitleArg = bShouldMute
			? LOCTEXT("Mute.ConfirmDialogTitle", "Confirm mute")
			: LOCTEXT("Unmute.ConfirmDialogTitle", "Confirm unmute");
		const FText ConfirmDialogDescriptionArg = bShouldMute
			? LOCTEXT("Mute.ConfirmDialogDescription", "Are you sure you want to mute the selected activities?")
			: LOCTEXT("Unmute.ConfirmDialogDescription", "Are you sure you want to unmute the selected activities?");
		
		SActivityDependencyDialog::Construct(
			SActivityDependencyDialog::FArguments()
				.Title(DialogTitle)
				.Description(Description)
				.PerformActionButtonLabel(ButtonLabel)
				.OnConfirmAction(InArgs._OnConfirmMute)
				.AdditionalColumns({ MutedColumn })
				.AnalyseActivities_Lambda([bShouldMute](const ConcertSyncCore::FActivityDependencyGraph& Graph, const TSet<FActivityID>& ActivityIds)
				{
					return bShouldMute
						? ConcertSyncCore::AnalyseActivityDependencies_TopDown(ActivityIds, Graph, true)
						: ConcertSyncCore::AnalyseActivityDependencies_BottomUp(ActivityIds, Graph, true);
				})
				// Improved UX: Don't show  muted activities when muting and unmuted activities when unmuting
				.ShouldShowActivity_Lambda([bShouldMute](const FConcertSyncActivity& Activity)
				{
					const bool bIsMuted = (Activity.Flags & EConcertSyncActivityFlags::Muted) != EConcertSyncActivityFlags::None;
					return bIsMuted != bShouldMute;
				})
				.GetFinalInclusionResultText_Lambda([this, bShouldMute](const int64 ActivityId, bool bIsChecked)
				{
					return bShouldMute
						? LOCTEXT("WillMute", "Will be muted.")
						: LOCTEXT("WillUnmute", "Will be unmuted.");
				})
				.ConfirmDialogTitle(ConfirmDialogTitleArg)
				.ConfirmDialogMessage(ConfirmDialogDescriptionArg),
			InSessionId, InSyncServer, MoveTemp(InBaseActivities)
		);
	}

	bool SMuteActivityDependenciesDialog::IsMuted(const int64 ActivityId)
	{
		if (TSharedPtr<IConcertSyncServer> PinnedSyncServer = SyncServer.Pin())
		{
			FConcertSyncActivity Activity;
			const bool bGotActivity = (*PinnedSyncServer->GetArchivedSessionDatabase(SessionId))->GetActivity(ActivityId, Activity);
			check(bGotActivity);
			
			const bool bIsMuted = (Activity.Flags & EConcertSyncActivityFlags::Muted) != EConcertSyncActivityFlags::None;
			return bIsMuted;
		}

		// Main window might have been closed destroying the server...
		return false;
	}
}

#undef LOCTEXT_NAMESPACE
