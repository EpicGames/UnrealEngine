// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SActivityDependencyDialog.h"

class FArchivedSessionHistoryController;
namespace UE::ConcertSyncCore
{
	struct FHistoryAnalysisResult;
}

namespace UE::MultiUserServer
{
	/** Displayed when a user asks to delete an activity. */
	class SMuteActivityDependenciesDialog : public SActivityDependencyDialog
	{
	public:
		enum class EMuteOperation
		{
			Mute,
			Unmute
		};
		
		SLATE_BEGIN_ARGS(SMuteActivityDependenciesDialog)
			: _MuteOperation(EMuteOperation::Mute)
		{}
			/** Called when the user confirms the muting of the activities.*/
			SLATE_EVENT(SActivityDependencyDialog::FConfirmDependencyAction, OnConfirmMute)

			/** Whether the activities should be muted or unmuted */
			SLATE_ARGUMENT(EMuteOperation, MuteOperation)
		SLATE_END_ARGS()

		/**
		 * @param InDependencyArgs Specifies which activities must be muted and which are optional.
		 */
		void Construct(const FArguments& InArgs, const FGuid& InSessionId, const TSharedRef<IConcertSyncServer>& InSyncServer, TSet<FActivityID> InBaseActivities);

	private:

		TWeakPtr<IConcertSyncServer> SyncServer;
		FGuid SessionId;

		bool IsMuted(const int64 ActivityId);
	};
}


