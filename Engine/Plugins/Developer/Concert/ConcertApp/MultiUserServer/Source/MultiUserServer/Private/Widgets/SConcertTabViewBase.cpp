// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertTabViewBase.h"

#include "Widgets/StatusBar/SConcertStatusBar.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

void SConcertTabViewBase::Construct(const FArguments& InArgs, FName InStatusBarId)
{
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(1.0f, 2.0f))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(1.0f, 2.0f)
			[
				InArgs._Content.Widget
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			[
				// This will cause a failing check if InStatusBarId is not unique
				SNew(SConcertStatusBar, InStatusBarId)
			]
		]
	];
}
