// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

/**
 * Utility base class for all content that is displayed in an entire major tab.
 * This class handles shared logic, such as creating a status bar.
 * 
 * If the content uses a tab manager, use SConcertTabViewWithManagedBase.
 */
class SConcertTabViewBase : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SConcertTabViewBase) {}
		/** The tab's content */
		SLATE_NAMED_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	/**
	 * @param InArgs
	 * @param InStatusBarId Unique ID needed for the status bar
	 */
	void Construct(const FArguments& InArgs, FName InStatusBarId);
};
