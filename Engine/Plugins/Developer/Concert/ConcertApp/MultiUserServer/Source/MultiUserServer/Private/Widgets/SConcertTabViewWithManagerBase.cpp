// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertTabViewWithManagerBase.h"

#include "Framework/Commands/UICommandList.h"
#include "Widgets/Util/SMultiUserIcons.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SConcertTabViewBase.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SConcertTabViewWithManagerBase"

void SConcertTabViewWithManagerBase::Construct(const FArguments& InArgs, FName InStatusBarId)
{
	check(InArgs._ConstructUnderWindow && InArgs._ConstructUnderMajorTab);
	SConcertTabViewBase::Construct(
		SConcertTabViewBase::FArguments()
		.Content()
		[
			InArgs._OverlayTabs.IsBound() ? InArgs._OverlayTabs.Execute(CreateTabs(InArgs)) : CreateTabs(InArgs)
		],
		InStatusBarId
		);
}

TSharedRef<SWidget> SConcertTabViewWithManagerBase::CreateTabs(const FArguments& InArgs)
{
	TabManager = FGlobalTabmanager::Get()->NewTabManager(InArgs._ConstructUnderMajorTab.ToSharedRef());
	TabManager->SetMainTab(InArgs._ConstructUnderMajorTab.ToSharedRef());
	
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout(InArgs._LayoutName);
	InArgs._CreateTabs.ExecuteIfBound(TabManager.ToSharedRef(), Layout);
	Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);
	TabManager->SetOnPersistLayout(
		FTabManager::FOnPersistLayout::CreateStatic(
			[](const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				if (InLayout->GetPrimaryArea().Pin().IsValid())
				{
					FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
				}
			}
		)
	);
	TSharedRef<SWidget> Result = TabManager->RestoreFrom(Layout, InArgs._ConstructUnderWindow).ToSharedRef();
	
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());
	FillInDefaultMenuItems(MenuBarBuilder);
	InArgs._CreateMenuBar.ExecuteIfBound(MenuBarBuilder);
	
	const TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();
	TabManager->SetAllowWindowMenuBar(true);
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);
	
	return Result;
}

void SConcertTabViewWithManagerBase::FillInDefaultMenuItems(FMenuBarBuilder MenuBarBuilder)
{
	MenuBarBuilder.AddPullDownMenu(
			LOCTEXT("WindowMenuLabel", "Window"),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(this, &SConcertTabViewWithManagerBase::FillWindowMenu),
			"Window"
		);
	MenuBarBuilder.AddPullDownMenu(
			LOCTEXT("DebugMenuLabel", "Debug"),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(this, &SConcertTabViewWithManagerBase::FillDebugMenu),
			"Debug"
		);
}

void SConcertTabViewWithManagerBase::FillWindowMenu(FMenuBuilder& MenuBuilder)
{
	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}

void SConcertTabViewWithManagerBase::FillDebugMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Icons", "View App Icons"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SConcertTabViewWithManagerBase::AddIconWindow)
			)
		);
}

void SConcertTabViewWithManagerBase::AddIconWindow()
{
	const FText AboutWindowTitle = LOCTEXT("UnrealMultiUserIcons", "Unreal Multi-User Server Icons");
	TSharedPtr<SWindow> AboutWindow = 
		SNew(SWindow)
		.Title( AboutWindowTitle )
		.ClientSize(FVector2D(720.f, 538.f))
		.SupportsMaximize(false) .SupportsMinimize(false)
		.SizingRule( ESizingRule::FixedSize )
		[
			SNew(SMultiUserIcons)
		];

	FSlateApplication::Get().AddWindow(AboutWindow.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE
