// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "SConcertTabViewBase.h"

class FMenuBarBuilder;

/** Base class for tab views that create sub-tabs. */
class SConcertTabViewWithManagerBase : public SConcertTabViewBase
{
public:

	DECLARE_DELEGATE_TwoParams(FCreateTabs, const TSharedRef<FTabManager>& /*TabManager*/, const TSharedRef<FTabManager::FLayout>& /*Layout*/);
	DECLARE_DELEGATE_OneParam(FCreateWindowMenu, FMenuBarBuilder&);
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOverlayTabs, const TSharedRef<SWidget>& /* Tabs */);
	
	SLATE_BEGIN_ARGS(SConcertTabViewWithManagerBase) {}
		/** Which major tab to construct the sub-tabs under. */
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, ConstructUnderMajorTab)
		/** The window in which the sub-tabs will be created. */
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ConstructUnderWindow)
		/** Callback for creating the sub-tabs */
		SLATE_EVENT(FCreateTabs, CreateTabs)
		/** Called to fill the major tab's menu bar */
		SLATE_EVENT(FCreateWindowMenu, CreateMenuBar)
		/** Optional function to overlay all the tabs (for example to overlay a message to execute a console command first) */
		SLATE_EVENT(FOverlayTabs, OverlayTabs)
		/** Name to give the layout. Important for saving config. */
		SLATE_ARGUMENT(FName, LayoutName)
	SLATE_END_ARGS()

	/**
	 * @param InArgs
	 * @param InStatusBarId Unique ID needed for the status bar
	 */
	void Construct(const FArguments& InArgs, FName InStatusBarId);

protected:

	TSharedPtr<FTabManager> GetTabManager() const { return TabManager; }
	
private:

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;
	
	TSharedRef<SWidget> CreateTabs(const FArguments& InArgs);
	void FillInDefaultMenuItems(FMenuBarBuilder MenuBarBuilder);
	void FillWindowMenu(FMenuBuilder& MenuBuilder);
	void FillDebugMenu(FMenuBuilder& MenuBuilder);

	void AddIconWindow();
};
