// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SessionTabs/Archived/ArchivedConcertSessionTab.h"

#include "ArchivedSessionHistoryController.h"
#include "ConcertLogGlobal.h"
#include "IConcertServer.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"

#include "HistoryEdition/ActivityNode.h"
#include "HistoryEdition/HistoryAnalysis.h"
#include "HistoryEdition/HistoryEdition.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "MultiUserServerModule.h"
#include "IConcertModule.h"
#include "Window/ModalWindowManager.h"
#include "Templates/NonNullPointer.h"
#include "Widgets/HistoryEdition/SDeleteActivityDependenciesDialog.h"
#include "Widgets/HistoryEdition/SActivityDependencyDialog.h"
#include "Widgets/SessionTabs/Archived/SConcertArchivedSessionTabView.h"
#include "Widgets/HistoryEdition/SMuteActivityDependenciesDialog.h"


#include "Algo/AllOf.h"
#include "Dialog/SMessageDialog.h"
#include "Settings/MultiUserServerUserPreferences.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SConcertTabViewBase.h"
#include "Widgets/Util/SDoNotShowAgainDialog.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FArchivedConcertSessionTab"

FArchivedConcertSessionTab::FArchivedConcertSessionTab(FGuid InspectedSessionID, TSharedRef<IConcertSyncServer> SyncServer, TAttribute<TSharedRef<SWindow>> ConstructUnderWindow)
	: FConcertSessionTabBase(InspectedSessionID, SyncServer)
	, InspectedSessionID(MoveTemp(InspectedSessionID))
	, SyncServer(MoveTemp(SyncServer))
	, ConstructUnderWindow(MoveTemp(ConstructUnderWindow))
{}

void FArchivedConcertSessionTab::CreateDockContent(const TSharedRef<SDockTab>& InDockTab)
{
	SEditableSessionHistory::FMakeSessionHistory MakeSessionHistory = SEditableSessionHistory::FMakeSessionHistory::CreateLambda([this](SSessionHistory::FArguments Arguments)
	{
		if (!HistoryController)
		{
			HistoryController = UE::MultiUserServer::CreateForInspector(InspectedSessionID, SyncServer, MoveTemp(Arguments));
		}
		return HistoryController->GetSessionHistory();
	});
	
	InDockTab->SetContent(
		SNew(SConcertArchivedSessionTabView, *GetTabId())
			.ConstructUnderMajorTab(InDockTab)
			.ConstructUnderWindow(ConstructUnderWindow.Get())
			.MakeSessionHistory(MoveTemp(MakeSessionHistory))
			.DeleteActivities_Raw(this, &FArchivedConcertSessionTab::OnRequestDeleteActivities)
			.CanDeleteActivities_Raw(this, &FArchivedConcertSessionTab::CanDeleteActivities)
			.MuteActivities_Raw(this, &FArchivedConcertSessionTab::OnRequestMuteActivities)
			.CanMuteActivities_Raw(this, &FArchivedConcertSessionTab::CanMuteActivities)
			.UnmuteActivities_Raw(this, &FArchivedConcertSessionTab::OnRequestUnmuteActivities)
			.CanUnmuteActivities_Raw(this, &FArchivedConcertSessionTab::CanUnmuteActivities)
		);
}

const FSlateBrush* FArchivedConcertSessionTab::GetTabIconBrush() const
{
	return FConcertFrontendStyle::Get()->GetBrush("Concert.ArchivedSession.Icon");
}

namespace UE::MultiUserServer::Private
{

FConcertSessionFilter BuildFilterFrom(const TSet<FActivityID>& ToDelete)
{
	FConcertSessionFilter Result;
	Result.ActivityIdsToExclude.Reserve(ToDelete.Num());
	for (const FActivityID ActivityID : ToDelete)
	{
		Result.ActivityIdsToExclude.Add(ActivityID);
	}
	return Result;
}

}

UE::ConcertSyncCore::FOperationErrorResult DeleteActivitiesInArchivedSession(const TSharedRef<IConcertServer>& Server, const FGuid& ArchivedSessionId, const TSet<FActivityID>& ToDelete)
{
	using namespace UE::ConcertSyncCore;

	const FString SessionId = ArchivedSessionId.ToString(EGuidFormats::DigitsWithHyphens);
	UE_LOGF(LogConcert, Log, "Deleting %d activities from session %ls...", ToDelete.Num(), *SessionId);
	TOptional<FString> ErrorMessage;
	ON_SCOPE_EXIT
	{
		if (ErrorMessage)
		{
			UE_LOGF(LogConcert, Error, "Finished deleting activities from %ls with error: %ls.", *SessionId, **ErrorMessage);
		}
		else
		{
			UE_LOGF(LogConcert, Log, "Finished deleting activities from %ls successfully.", *SessionId);
		}
	};

	const TOptional<FConcertSessionInfo> DeletedSessionInfo = Server->GetArchivedSessionInfo(ArchivedSessionId);
	if (!DeletedSessionInfo)
	{
		ErrorMessage = FString::Printf(TEXT("Session ID %s does not resolve to any archived session!"), *ArchivedSessionId.ToString());
		return FOperationErrorResult::MakeError(FText::FromString(*ErrorMessage));
	}

	// Restore the session while skipping all to be deleted activities
	FText FailureReason;
	const FConcertSessionFilter Filter = UE::MultiUserServer::Private::BuildFilterFrom(ToDelete);
	// Do not use same name in case user already created a live session from this archived session (would fail in that case) 
	FConcertSessionInfo OverrideInfo = *DeletedSessionInfo;
	OverrideInfo.SessionName = DeletedSessionInfo->SessionName + FGuid::NewGuid().ToString();
	const TSharedPtr<IConcertServerSession> LiveSession = Server->RestoreSession(ArchivedSessionId, OverrideInfo, Filter, FailureReason);
	if (!LiveSession)
	{
		ErrorMessage = FailureReason.ToString();
		return FOperationErrorResult::MakeError(FailureReason);
	}
	ON_SCOPE_EXIT
	{
		Server->DestroySession(LiveSession->GetId(), FailureReason);
	};

	// The archived session must be removed before it can be overwritten
	if (!Server->DestroySession(ArchivedSessionId, FailureReason))
	{
		ErrorMessage = FailureReason.ToString();
		return FOperationErrorResult::MakeError(FailureReason); 
	}

	if (!Server->ArchiveSession(LiveSession->GetId(), DeletedSessionInfo->SessionName, Filter, FailureReason, ArchivedSessionId).IsValid())
	{
		ErrorMessage = FailureReason.ToString();
		return FOperationErrorResult::MakeError(FailureReason); 
	}

	return FOperationErrorResult::MakeSuccess(); 
}


void FArchivedConcertSessionTab::OnRequestDeleteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const
{
	using namespace UE::ConcertSyncCore;
	using namespace UE::MultiUserServer;
	
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> SessionDatabase = SyncServer->GetArchivedSessionDatabase(InspectedSessionID))
	{
		TSet<FActivityID> RequestedForDelete;
		Algo::Transform(ActivitiesToDelete, RequestedForDelete, [](const TSharedRef<FConcertSessionActivity>& Activity)
		{
			return Activity->Activity.ActivityId;
		});

		const TWeakPtr<const FArchivedConcertSessionTab> WeakTabThis = SharedThis(this);
		const TSharedRef<SDeleteActivityDependenciesDialog> Dialog = SNew(SDeleteActivityDependenciesDialog, InspectedSessionID, SyncServer, MoveTemp(RequestedForDelete))
			.OnConfirmDeletion_Lambda([WeakTabThis](const TSet<FActivityID>& Selection)
			{
				// Because the dialog is non-modal, the user may have closed the program in the mean time
				if (const TSharedPtr<const FArchivedConcertSessionTab> PinnedThis = WeakTabThis.Pin())
				{
					const FOperationErrorResult ErrorResult = DeleteActivitiesInArchivedSession(PinnedThis->SyncServer->GetConcertServer(), PinnedThis->InspectedSessionID, Selection);
					if (ErrorResult.HadError())
					{
						UE_LOGF(LogConcert, Error, "Failed to delete activities from session %ls: %ls", *PinnedThis->InspectedSessionID.ToString(), *ErrorResult.ErrorMessage->ToString());
						
						const TSharedRef<SMessageDialog> ErrorDialog = SNew(SMessageDialog)
							.Title(LOCTEXT("ErrorDeletingSessions", "Error deleting sessions"))
							.Message(*ErrorResult.ErrorMessage)
							.Buttons({
								SMessageDialog::FButton(LOCTEXT("Ok", "Ok"))
								.SetPrimary(true)
							});
						ErrorDialog->Show();
					}
					else
					{
						// The list needs to be refreshed after the delete operation
						PinnedThis->HistoryController->ReloadActivities();
					}
				}
			});
		
		FConcertServerUIModule::Get()
			.GetModalWindowManager()
			->ShowFakeModalWindow(Dialog);
	}
}

FCanPerformActionResult FArchivedConcertSessionTab::CanDeleteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const
{
	const bool bOnlyPackagesAndTransactions =  Algo::AllOf(ActivitiesToDelete, [](const TSharedRef<FConcertSessionActivity>& Activity)
	{
		return Activity->Activity.EventType == EConcertSyncActivityEventType::Package || Activity->Activity.EventType == EConcertSyncActivityEventType::Transaction;
	});
	if (!bOnlyPackagesAndTransactions)
	{
		return FCanPerformActionResult::No(LOCTEXT("Delete.CanDeleteActivity.OnlyPackagesAndTransactionsReason", "Only package and transaction activities can be deleted (the current selection includes other activity types)."));
	}

	return FCanPerformActionResult::Yes();
}

void FArchivedConcertSessionTab::OnRequestMuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToMute) const
{
	using namespace UE::MultiUserServer;
	
	if (UMultiUserServerUserPreferences::GetSettings()->bWarnUserAboutMuting)
	{
		const TSharedRef<SDoNotShowAgainDialog> Dialog =
		SNew(SDoNotShowAgainDialog)
			.Title(LOCTEXT("MuteTitle", "Caution: Muting"))
			.Buttons({
				SCustomDialog::FButton(LOCTEXT("Continue", "Continue"))
					.SetPrimary(true)
					.SetOnClicked(FSimpleDelegate::CreateLambda([this, ActivitiesToMute]()
					{
						constexpr bool bShouldMute = true;
						SpawnDialogForMutingOrUnmuting(bShouldMute, ActivitiesToMute);
					})),
				SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
			})
			.DoNotShowAgainCallback(SDoNotShowAgainDialog::FOnClosed::CreateLambda([](bool bDoNotShowAgain)
			{
				UMultiUserServerUserPreferences* UserPreferences = UMultiUserServerUserPreferences::GetSettings();
				UserPreferences->bWarnUserAboutMuting = !bDoNotShowAgain;
				UserPreferences->SaveConfig();
			}))
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MuteWarning", "Muted activities are not sent to clients. Activities you select are analysed for dependencies.\nExample: Muting the creation of a map also mutes editing an actor in that level.\n\nMuting is not always safe.\n\nDependency analysis cannot detect all cases:\nsometimes transactions may depend on other transaction activities having occured before.\nThis cannot be detected."))
			];

		FConcertServerUIModule::Get()
			.GetModalWindowManager()
			->ShowFakeModalWindow(Dialog);
	}
	else
	{
		constexpr bool bShouldMute = true;
		SpawnDialogForMutingOrUnmuting(bShouldMute, ActivitiesToMute);
	}	
}

FCanPerformActionResult FArchivedConcertSessionTab::CanMuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const
{
	const bool bOnlyPackagesAndTransactions = Algo::AllOf(ActivitiesToDelete, [](const TSharedRef<FConcertSessionActivity>& Activity)
	{
		return Activity->Activity.EventType == EConcertSyncActivityEventType::Package || Activity->Activity.EventType == EConcertSyncActivityEventType::Transaction;
	});
	if (!bOnlyPackagesAndTransactions)
	{
		return FCanPerformActionResult::No(LOCTEXT("Mute.CanMuteActivity.OnlyPackagesAndTransactionsReason", "Only package and transaction activities can be deleted (the current selection includes other activity types)."));
	}

	const bool bAreAllUnmuted = Algo::AllOf(ActivitiesToDelete, [](const TSharedRef<FConcertSessionActivity>& Activity)
	{
		return (Activity->Activity.Flags & EConcertSyncActivityFlags::Muted) == EConcertSyncActivityFlags::None;
	});
	if (!bAreAllUnmuted)
	{
		return FCanPerformActionResult::No(LOCTEXT("Mute.CanMuteActivity.NotAllUnmuted", "Some of the selected activities are muted."));
	}

	return FCanPerformActionResult::Yes();
}

void FArchivedConcertSessionTab::OnRequestUnmuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToUnmute) const
{
	using namespace UE::MultiUserServer;
	
	if (UMultiUserServerUserPreferences::GetSettings()->bWarnUserAboutUnmuting)
	{
		const TSharedRef<SDoNotShowAgainDialog> Dialog =
		SNew(SDoNotShowAgainDialog)
			.Title(LOCTEXT("UnmuteTitle", "Caution: Unmuting"))
			.Buttons({
				SCustomDialog::FButton(LOCTEXT("Continue", "Continue"))
					.SetPrimary(true)
					.SetOnClicked(FSimpleDelegate::CreateLambda([this, ActivitiesToUnmute]()
					{
						constexpr bool bShouldMute = false;
						SpawnDialogForMutingOrUnmuting(bShouldMute, ActivitiesToUnmute);
					})),
				SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
			})
			.DoNotShowAgainCallback(SDoNotShowAgainDialog::FOnClosed::CreateLambda([](bool bDoNotShowAgain)
			{
				UMultiUserServerUserPreferences* UserPreferences = UMultiUserServerUserPreferences::GetSettings();
				UserPreferences->bWarnUserAboutUnmuting = !bDoNotShowAgain;
				UserPreferences->SaveConfig();
			}))
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UnmuteWarning", "Unmuting is not always safe\nPay attention if this session had muted activities, was unarchived, and now you're unmuting said activities again.\n\nExample of conflict:\n1. Create package Foo\nNow mute 1 and unarchive session.\n1. Create package Foo (muted)\n2. Create package Foo\nArchive the session, and try to unmute 1. You'd end up with a history that creates Foo twice, which is not valid.\n\nSimilar invalid cases can happen with transaction activities."))
			];

		FConcertServerUIModule::Get()
			.GetModalWindowManager()
			->ShowFakeModalWindow(Dialog);
	}
	else
	{
		constexpr bool bShouldMute = false;
		SpawnDialogForMutingOrUnmuting(bShouldMute, ActivitiesToUnmute);
	}	
}

FCanPerformActionResult FArchivedConcertSessionTab::CanUnmuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const
{
	const bool bOnlyPackagesAndTransactions = Algo::AllOf(ActivitiesToDelete, [](const TSharedRef<FConcertSessionActivity>& Activity)
	{
		return Activity->Activity.EventType == EConcertSyncActivityEventType::Package || Activity->Activity.EventType == EConcertSyncActivityEventType::Transaction;
	});
	if (!bOnlyPackagesAndTransactions)
	{
		return FCanPerformActionResult::No(LOCTEXT("Mute.CanMuteActivity.OnlyPackagesAndTransactionsReason", "Only package and transaction activities can be deleted (the current selection includes other activity types)."));
	}

	const bool bAreAllMuted = Algo::AllOf(ActivitiesToDelete, [](const TSharedRef<FConcertSessionActivity>& Activity)
	{
		return (Activity->Activity.Flags & EConcertSyncActivityFlags::Muted) != EConcertSyncActivityFlags::None;
	});
	if (!bAreAllMuted)
	{
		return FCanPerformActionResult::No(LOCTEXT("Mute.CanMuteActivity.NotAllMuted", "Some of the selected activities are not muted."));
	}

	return FCanPerformActionResult::Yes();
}

void FArchivedConcertSessionTab::SpawnDialogForMutingOrUnmuting(bool bShouldMute, const TSet<TSharedRef<FConcertSessionActivity>>& Activities) const
{
	using namespace UE::ConcertSyncCore;
	using namespace UE::MultiUserServer;
	
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> SessionDatabase = SyncServer->GetArchivedSessionDatabase(InspectedSessionID))
	{
		TSet<FActivityID> Requested;
		Algo::Transform(Activities, Requested, [](const TSharedRef<FConcertSessionActivity>& Activity)
		{
			return Activity->Activity.ActivityId;
		});

		const TWeakPtr<const FArchivedConcertSessionTab> WeakTabThis = SharedThis(this);
		const TSharedRef<SMuteActivityDependenciesDialog> Dialog = SNew(SMuteActivityDependenciesDialog, InspectedSessionID, SyncServer, MoveTemp(Requested))
			.MuteOperation(bShouldMute ? SMuteActivityDependenciesDialog::EMuteOperation::Mute : SMuteActivityDependenciesDialog::EMuteOperation::Unmute)
			.OnConfirmMute_Lambda([WeakTabThis, bShouldMute, this](const TSet<FActivityID>& Selection)
			{
				// Because the dialog is non-modal, the user may have closed the program in the mean time
				if (const TSharedPtr<const FArchivedConcertSessionTab> PinnedThis = WeakTabThis.Pin())
				{
					const bool bSuccess = (*PinnedThis->SyncServer->GetArchivedSessionDatabase(PinnedThis->InspectedSessionID))->SetActivities(Selection, [bShouldMute](FConcertSyncActivity& Activity)
					{
						if (bShouldMute)
						{
							Activity.Flags |= EConcertSyncActivityFlags::Muted;
						}
						else
						{
							Activity.Flags &= ~EConcertSyncActivityFlags::Muted;
						}
					});
					if (bSuccess)
					{
						// Need to update model for history widget
						HistoryController->ReloadActivities();
					}
					else
					{
						UE_LOGF(LogConcert, Error, "Failed to mute activities from session %ls", *PinnedThis->InspectedSessionID.ToString(EGuidFormats::DigitsWithHyphens));
						
						const TSharedRef<SMessageDialog> ErrorDialog = SNew(SMessageDialog)
							.Title(LOCTEXT("ErrorMutingSessions.Title", "Error muting sessions"))
							.Message(LOCTEXT("ErrorMutingSessions.Description", "Error writing to database"))
							.Buttons({
								SMessageDialog::FButton(LOCTEXT("Ok", "Ok"))
								.SetPrimary(true)
							});
						ErrorDialog->Show();
					}
				}
			});
		
		FConcertServerUIModule::Get()
			.GetModalWindowManager()
			->ShowFakeModalWindow(Dialog);
	}
}

#undef LOCTEXT_NAMESPACE
