// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Widgets/SessionTabs/ConcertSessionTabBase.h"

class SWindow;
struct FCanPerformActionResult;

struct FConcertSessionActivity;
class FArchivedSessionHistoryController;
class SConcertArchivedSessionTabView;

/** Manages the tab for an archived session.  */
class FArchivedConcertSessionTab : public FConcertSessionTabBase
{
public:

	FArchivedConcertSessionTab(FGuid InspectedSessionID, TSharedRef<IConcertSyncServer> SyncServer, TAttribute<TSharedRef<SWindow>> ConstructUnderWindow);

protected:

	//~ Begin FAbstractConcertSessionTab Interface
	virtual void CreateDockContent(const TSharedRef<SDockTab>& InDockTab) override;
	virtual const FSlateBrush* GetTabIconBrush() const override;
	virtual void OnOpenTab() override {}
	//~ End FAbstractConcertSessionTab Interface
	
private:

	/** The inspected session's ID */
	const FGuid InspectedSessionID;

	/** Used later to construct Inspector */
	const TSharedRef<IConcertSyncServer> SyncServer;
	
	/** Used later to obtain the window into which to add the tab */
	const TAttribute<TSharedRef<SWindow>> ConstructUnderWindow;
	
	TSharedPtr<FArchivedSessionHistoryController> HistoryController;
	
	/** Displays session */
	TSharedPtr<SConcertArchivedSessionTabView> Inspector;

	void OnRequestDeleteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const;
	FCanPerformActionResult CanDeleteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const;
	
	void OnRequestMuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToMute) const;
	FCanPerformActionResult CanMuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const;
	
	void OnRequestUnmuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const;
	FCanPerformActionResult CanUnmuteActivities(const TSet<TSharedRef<FConcertSessionActivity>>& ActivitiesToDelete) const;

	void SpawnDialogForMutingOrUnmuting(bool bShouldMute, const TSet<TSharedRef<FConcertSessionActivity>>& Activities) const;
};
