// Copyright Epic Games, Inc. All Rights Reserved.

#include "ArchivedSessionHistoryController.h"

#include "Settings/MultiUserServerColumnVisibilitySettings.h"
#include "Templates/NonNullPointer.h"

FArchivedSessionHistoryController::FArchivedSessionHistoryController(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments)
	: FServerSessionHistoryControllerBase(MoveTemp(SessionId), MoveTemp(Arguments))
	, SyncServer(MoveTemp(SyncServer))
{
	ReloadActivities();
}

TOptional<FConcertSyncSessionDatabaseNonNullPtr> FArchivedSessionHistoryController::GetSessionDatabase(const FGuid& InSessionId) const
{
	return SyncServer->GetArchivedSessionDatabase(InSessionId);
}

namespace UE::MultiUserServer
{
	namespace Private
	{
		static SSessionHistory::FArguments MakeArgumentsForInspector(SSessionHistory::FArguments&& InArgs)
		{
			return InArgs
				.ColumnVisibilitySnapshot(UMultiUserServerColumnVisibilitySettings::GetSettings()->GetArchivedActivityBrowserColumnVisibility())
				.SaveColumnVisibilitySnapshot_Lambda([](const FColumnVisibilitySnapshot& Snapshot)
				{
					UMultiUserServerColumnVisibilitySettings::GetSettings()->SetArchivedActivityBrowserColumnVisibility(Snapshot);
				});
		}

		class FInspectorSessionHistoryController : public FArchivedSessionHistoryController
		{
		public:
			FInspectorSessionHistoryController(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments)
				: FArchivedSessionHistoryController(MoveTemp(SessionId), MoveTemp(SyncServer), MakeArgumentsForInspector(MoveTemp(Arguments)))
			{
				UMultiUserServerColumnVisibilitySettings::GetSettings()->OnArchivedActivityBrowserColumnVisibility().AddRaw(this, &FInspectorSessionHistoryController::OnActivityListColumnVisibilitySettingsUpdated);
			}
		
			virtual ~FInspectorSessionHistoryController() override
			{
				if (UMultiUserServerColumnVisibilitySettings* Settings = UMultiUserServerColumnVisibilitySettings::GetSettings(); IsValid(Settings))
				{
					Settings->OnArchivedActivityBrowserColumnVisibility().RemoveAll(this);
				}
			}
		
			void OnActivityListColumnVisibilitySettingsUpdated(const FColumnVisibilitySnapshot& NewValue)
			{
				GetSessionHistory()->OnColumnVisibilitySettingsChanged(NewValue);
			}
		};
		
		static SSessionHistory::FArguments MakeArgumentsForDeleteDialog(SSessionHistory::FArguments&& InArgs)
		{
			return InArgs
				.ColumnVisibilitySnapshot(UMultiUserServerColumnVisibilitySettings::GetSettings()->GetDeleteActivityDialogColumnVisibility())
				.SaveColumnVisibilitySnapshot_Lambda([](const FColumnVisibilitySnapshot& Snapshot)
				{
					UMultiUserServerColumnVisibilitySettings::GetSettings()->SetDeleteActivityDialogColumnVisibility(Snapshot);
				});
		}

		class FDeleteDialogSessionHistoryController : public FArchivedSessionHistoryController
		{
		public:
			
			FDeleteDialogSessionHistoryController(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments)
				: FArchivedSessionHistoryController(MoveTemp(SessionId), MoveTemp(SyncServer), MakeArgumentsForDeleteDialog(MoveTemp(Arguments)))
			{
				UMultiUserServerColumnVisibilitySettings::GetSettings()->OnDeleteActivityDialogColumnVisibility().AddRaw(this, &FDeleteDialogSessionHistoryController::OnActivityListColumnVisibilitySettingsUpdated);
			}
		
			virtual ~FDeleteDialogSessionHistoryController() override
			{
				if (UMultiUserServerColumnVisibilitySettings* Settings = UMultiUserServerColumnVisibilitySettings::GetSettings(); IsValid(Settings))
				{
					Settings->OnDeleteActivityDialogColumnVisibility().RemoveAll(this);
				}
			}
		
			void OnActivityListColumnVisibilitySettingsUpdated(const FColumnVisibilitySnapshot& NewValue)
			{
				GetSessionHistory()->OnColumnVisibilitySettingsChanged(NewValue);
			}
		};
	}

	TSharedPtr<FArchivedSessionHistoryController> CreateForInspector(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments)
	{
		return MakeShared<Private::FInspectorSessionHistoryController>(MoveTemp(SessionId), MoveTemp(SyncServer), MoveTemp(Arguments));
	}

	TSharedPtr<FArchivedSessionHistoryController> CreateForDeletionDialog(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments)
	{
		return MakeShared<Private::FDeleteDialogSessionHistoryController>(MoveTemp(SessionId), MoveTemp(SyncServer), MoveTemp(Arguments));
	}
}


