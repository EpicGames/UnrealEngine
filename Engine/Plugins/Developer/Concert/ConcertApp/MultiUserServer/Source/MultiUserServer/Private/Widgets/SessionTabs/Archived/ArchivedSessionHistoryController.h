// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SessionTabs/ServerSessionHistoryControllerBase.h"

/** Manages SSessionHistory by using IConcertSyncServer::GetArchivedSessionDatabase for retrieving the session database. */
class FArchivedSessionHistoryController : public FServerSessionHistoryControllerBase
{
protected: // Protected to encourage the use the below functions (CreateForInspector, etc.) instead of MakeShared
	
	FArchivedSessionHistoryController(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments);

	//~ Begin FServerSessionHistoryControllerBase Interface
	virtual TOptional<FConcertSyncSessionDatabaseNonNullPtr> GetSessionDatabase(const FGuid& InSessionId) const override;
	//~ End FServerSessionHistoryControllerBase Interface

private:

	const TSharedRef<IConcertSyncServer> SyncServer;
};

namespace UE::MultiUserServer
{
	/** Creates a controller that is supposed to use UMultiUserServerUserSettings::GetArchivedActivityBrowserColumnVisibility  */
	TSharedPtr<FArchivedSessionHistoryController> CreateForInspector(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments);

	/** Creates a controller that is supposed to use  UMultiUserServerUserSettings::GetDeleteActivityDialogColumnVisibility*/
	TSharedPtr<FArchivedSessionHistoryController> CreateForDeletionDialog(FGuid SessionId, TSharedRef<IConcertSyncServer> SyncServer, SSessionHistory::FArguments Arguments);
}
