// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertArchivedSessionTabView.h"

#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "SConcertArchivedSessionInspector"

const FName SConcertArchivedSessionTabView::HistoryTabId("HistoryTabId");

void SConcertArchivedSessionTabView::Construct(const FArguments& InArgs, FName InStatusBarID)
{
	check(InArgs._MakeSessionHistory.IsBound() && InArgs._CanDeleteActivities.IsBound() && InArgs._CanMuteActivities.IsBound());
	SConcertTabViewWithManagerBase::Construct(
		SConcertTabViewWithManagerBase::FArguments()
		.ConstructUnderWindow(InArgs._ConstructUnderWindow)
		.ConstructUnderMajorTab(InArgs._ConstructUnderMajorTab)
		.CreateTabs(FCreateTabs::CreateLambda([this, &InArgs](const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout)
		{
			CreateTabs(InTabManager, InLayout, InArgs);
		}))
		.LayoutName("ConcertArchivedSessionInspector_v0.1"),
		InStatusBarID
	);
}

void SConcertArchivedSessionTabView::CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FArguments& InArgs)
{
	const TSharedRef<FWorkspaceItem> WorkspaceItem = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("ArchiveSession", "Archived Session"));
	
	InTabManager->RegisterTabSpawner(HistoryTabId, FOnSpawnTab::CreateSP(this, &SConcertArchivedSessionTabView::SpawnActivityHistory,
		InArgs._MakeSessionHistory,
		InArgs._CanDeleteActivities,
		InArgs._DeleteActivities,
		InArgs._CanMuteActivities,
		InArgs._MuteActivities,
		InArgs._CanUnmuteActivities,
		InArgs._UnmuteActivities
		))
		.SetDisplayName(LOCTEXT("ActivityHistoryLabel", "History"))
		.SetGroup(WorkspaceItem);
	InLayout->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(HistoryTabId, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
		);
}

TSharedRef<SDockTab> SConcertArchivedSessionTabView::SpawnActivityHistory(
	const FSpawnTabArgs& Args,
	SEditableSessionHistory::FMakeSessionHistory MakeSessionHistory,
	SEditableSessionHistory::FCanPerformActionOnActivities CanDeleteActivities,
	SEditableSessionHistory::FRequestActivitiesAction DeleteActivities,
	SEditableSessionHistory::FCanPerformActionOnActivities CanMuteActivities,
	SEditableSessionHistory::FRequestActivitiesAction MuteActivities,
	SEditableSessionHistory::FCanPerformActionOnActivities CanUnmuteActivities,
	SEditableSessionHistory::FRequestActivitiesAction UnmuteActivities)
{
	SessionHistory = SNew(SEditableSessionHistory)
		.MakeSessionHistory(MoveTemp(MakeSessionHistory))
		.CanDeleteActivities(MoveTemp(CanDeleteActivities))
		.DeleteActivities(MoveTemp(DeleteActivities))
		.CanMuteActivities(MoveTemp(CanMuteActivities))
		.MuteActivities(MoveTemp(MuteActivities))
		.CanUnmuteActivities(MoveTemp(CanUnmuteActivities))
		.UnmuteActivities(MoveTemp(UnmuteActivities));
	return SNew(SDockTab)
		.Label(LOCTEXT("ActivityHistoryLabel", "History"))
		.TabRole(PanelTab)
		[
			SessionHistory.ToSharedRef()
		]; 
}

#undef LOCTEXT_NAMESPACE
