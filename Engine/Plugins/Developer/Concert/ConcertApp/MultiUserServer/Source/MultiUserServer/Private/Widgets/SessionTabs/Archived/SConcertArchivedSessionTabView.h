// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Session/History/SEditableSessionHistory.h"
#include "Widgets/SConcertTabViewWithManagerBase.h"

class SDockTab;
class SSessionHistory;

/** Designed as content for a tab. Displays information about an archived session. */
class SConcertArchivedSessionTabView : public SConcertTabViewWithManagerBase
{
public:
	
	static const FName HistoryTabId;

	SLATE_BEGIN_ARGS(SConcertArchivedSessionTabView)
	{}
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, ConstructUnderMajorTab)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ConstructUnderWindow)
		SLATE_EVENT(SEditableSessionHistory::FMakeSessionHistory, MakeSessionHistory)
	
		SLATE_EVENT(SEditableSessionHistory::FCanPerformActionOnActivities, CanDeleteActivities)
		SLATE_EVENT(SEditableSessionHistory::FRequestActivitiesAction, DeleteActivities)
	
		SLATE_EVENT(SEditableSessionHistory::FCanPerformActionOnActivities, CanMuteActivities)
		SLATE_EVENT(SEditableSessionHistory::FRequestActivitiesAction, MuteActivities)
	
		SLATE_EVENT(SEditableSessionHistory::FCanPerformActionOnActivities, CanUnmuteActivities)
		SLATE_EVENT(SEditableSessionHistory::FRequestActivitiesAction, UnmuteActivities)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FName InStatusBarID);

private:

	TSharedPtr<SEditableSessionHistory> SessionHistory;
	
	void CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FArguments& InArgs);
	TSharedRef<SDockTab> SpawnActivityHistory(const FSpawnTabArgs& Args,
		SEditableSessionHistory::FMakeSessionHistory FMakeSessionHistory,
		SEditableSessionHistory::FCanPerformActionOnActivities CanDeleteActivities,
		SEditableSessionHistory::FRequestActivitiesAction DeleteActivities,
		SEditableSessionHistory::FCanPerformActionOnActivities CanMuteActivities,
		SEditableSessionHistory::FRequestActivitiesAction MuteActivities,
		SEditableSessionHistory::FCanPerformActionOnActivities CanUnmuteActivities,
		SEditableSessionHistory::FRequestActivitiesAction UnmuteActivities
		);
};
