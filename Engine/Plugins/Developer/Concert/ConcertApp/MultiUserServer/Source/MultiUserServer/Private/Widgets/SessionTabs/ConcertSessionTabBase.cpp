// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSessionTabBase.h"

#include "IConcertModule.h"
#include "IConcertServer.h"
#include "IConcertSession.h"
#include "IConcertSyncServer.h"
#include "Window/ConcertServerTabs.h"

#include "Widgets/Docking/SDockTab.h"

namespace UE::ConcertServerUI::Private
{
	static TOptional<FString> GetSessionName(const TSharedRef<IConcertSyncServer>& SyncServer, const FGuid SessionId)
	{
		if (const TSharedPtr<IConcertServerSession> LiveSessionInfo = SyncServer->GetConcertServer()->GetLiveSession(SessionId))
		{
			return LiveSessionInfo->GetSessionInfo().SessionName;
		}
		if (const TOptional<FConcertSessionInfo> ArchivedSessionInfo = SyncServer->GetConcertServer()->GetArchivedSessionInfo(SessionId))
		{
			return ArchivedSessionInfo->SessionName;
		}
		return {};
	}
}

FConcertSessionTabBase::FConcertSessionTabBase(FGuid InspectedSessionID, TSharedRef<IConcertSyncServer> SyncServer)
	: InspectedSessionID(MoveTemp(InspectedSessionID))
	, SyncServer(MoveTemp(SyncServer))
{}

FConcertSessionTabBase::~FConcertSessionTabBase()
{
	const TSharedRef<FGlobalTabmanager>& TabManager = FGlobalTabmanager::Get();
	const FTabId TabId { *GetTabId() };
	if (const TSharedPtr<SDockTab> TabStack = TabManager->FindExistingLiveTab(TabId))
	{
		TabStack->RequestCloseTab();
	}
}

void FConcertSessionTabBase::OpenSessionTab()
{
	const TSharedRef<FGlobalTabmanager>& TabManager = FGlobalTabmanager::Get();
	const FTabId TabId { *GetTabId() };

	EnsureInitDockTab();
	if (TabManager->FindExistingLiveTab(TabId))
	{
		TabManager->DrawAttention(DockTab.ToSharedRef());
	}
	else
	{
		const FTabManager::FLastMajorOrNomadTab Search(ConcertServerTabs::GetSessionBrowserTabId());
		TabManager->InsertNewDocumentTab(*GetTabId(), Search, DockTab.ToSharedRef());

		OnOpenTab();
	}
}

void FConcertSessionTabBase::EnsureInitDockTab()
{
	if (!DockTab.IsValid())
	{
		DockTab = SNew(SDockTab)
			.Label_Lambda([this]()
			{
				return FText::FromString(
					UE::ConcertServerUI::Private::GetSessionName(SyncServer, GetSessionID())
						.Get(TEXT("Error getting name"))
					);
			})
			.TabRole(MajorTab);

		DockTab->SetTabIcon(GetTabIconBrush());
		CreateDockContent(DockTab.ToSharedRef());
	}
}
