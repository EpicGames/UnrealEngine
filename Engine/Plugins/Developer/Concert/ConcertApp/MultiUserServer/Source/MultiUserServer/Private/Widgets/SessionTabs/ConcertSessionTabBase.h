// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

class IConcertSyncServer;
class SDockTab;
class SWindow;
struct FSlateBrush;

/** Shared functionality for a tab that manages a session */
class FConcertSessionTabBase : public TSharedFromThis<FConcertSessionTabBase>
{
public:

    FConcertSessionTabBase(FGuid InspectedSessionID, TSharedRef<IConcertSyncServer> SyncServer);
	virtual ~FConcertSessionTabBase();
	
	/** Opens or draws attention to the given tab */
	void OpenSessionTab();
	
protected:

	virtual void CreateDockContent(const TSharedRef<SDockTab>& DockTab) = 0;
	virtual const FSlateBrush* GetTabIconBrush() const = 0;
	virtual void OnOpenTab() = 0;
	
	FGuid GetSessionID() const { return InspectedSessionID; }
	/** Generates a tab ID for FTabManager::InsertNewDocumentTab */
	FString GetTabId() const { return GetSessionID().ToString(); }
	const TSharedRef<IConcertSyncServer>& GetSyncServer() const { return SyncServer; }
	
private:

	/** The ID of the session this tab is managing */
	FGuid InspectedSessionID;
	
	/** Used to look up session name */
	const TSharedRef<IConcertSyncServer> SyncServer;
	
	/** The tab containing the UI for InspectedSession */
	TSharedPtr<SDockTab> DockTab;

	/** Inits DockTab if it has not yet been inited */
	void EnsureInitDockTab();
};
