// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveConcertSessionTab.h"

#include "IConcertSession.h"
#include "LiveServerSessionHistoryController.h"
#include "PackageViewer/ConcertSessionPackageViewerController.h"
#include "SConcertLiveSessionTabView.h"

#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SConcertTabViewBase.h"

FLiveConcertSessionTab::FLiveConcertSessionTab(TSharedRef<IConcertServerSession> InInspectedSession, TSharedRef<IConcertSyncServer> InSyncServer, TAttribute<TSharedRef<SWindow>> InConstructUnderWindow, FShowConnectedClients InOnConnectedClientsClicked)
	: FConcertSessionTabBase(InInspectedSession->GetSessionInfo().SessionId, InSyncServer)
	, InspectedSession(MoveTemp(InInspectedSession))
	, ConstructUnderWindow(MoveTemp(InConstructUnderWindow))
	, OnConnectedClientsClicked(MoveTemp(InOnConnectedClientsClicked))
	, SessionHistoryController(MakeShared<FLiveServerSessionHistoryController>(InspectedSession, InSyncServer))
	, PackageViewerController(MakeShared<FConcertSessionPackageViewerController>(InspectedSession, InSyncServer))
{}

void FLiveConcertSessionTab::CreateDockContent(const TSharedRef<SDockTab>& InDockTab)
{
	const SConcertLiveSessionTabView::FRequiredWidgets WidgetArgs
	{
		InDockTab,
		ConstructUnderWindow.Get(),
		SessionHistoryController->GetSessionHistory(),
		PackageViewerController->GetPackageViewer()
	};
	InDockTab->SetContent(
		SNew(SConcertLiveSessionTabView, WidgetArgs, *GetTabId())
		.OnConnectedClientsClicked_Lambda([this]()
		{
			OnConnectedClientsClicked.ExecuteIfBound(InspectedSession);
		}));
}

const FSlateBrush* FLiveConcertSessionTab::GetTabIconBrush() const
{
	return FConcertFrontendStyle::Get()->GetBrush("Concert.ActiveSession.Icon");
}

void FLiveConcertSessionTab::OnOpenTab()
{
	SessionHistoryController->ReloadActivities();
	PackageViewerController->ReloadActivities();
}

