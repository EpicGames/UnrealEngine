// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Widgets/SessionTabs/ConcertSessionTabBase.h"

class FConcertSessionPackageViewerController;
class FLiveServerSessionHistoryController;
class IConcertSyncServer;
class IConcertServerSession;
class SDockTab;
class SWindow;

/**
 * Manages the tab that contains the UI for a session.
 * It has access to the controllers and views needed for displaying a session.
 */
class FLiveConcertSessionTab : public FConcertSessionTabBase
{
public:

	DECLARE_DELEGATE_OneParam(FShowConnectedClients, const TSharedRef<IConcertServerSession>&);

	FLiveConcertSessionTab(TSharedRef<IConcertServerSession> InspectedSession,
		TSharedRef<IConcertSyncServer> SyncServer,
		TAttribute<TSharedRef<SWindow>> ConstructUnderWindow,
		FShowConnectedClients OnConnectedClientsClicked
		);

protected:

	//~ Begin FAbstractConcertSessionTab Interface
	virtual void CreateDockContent(const TSharedRef<SDockTab>& InDockTab) override;
	virtual const FSlateBrush* GetTabIconBrush() const override;
	virtual void OnOpenTab() override;
	//~ End FAbstractConcertSessionTab Interface
	
private:
	
	/** The session being inspected */
	const TSharedRef<IConcertServerSession> InspectedSession;

	/** Used to determine owning window during construction */
	TAttribute<TSharedRef<SWindow>> ConstructUnderWindow;

	/** Shows all connected clients for this session */
	FShowConnectedClients OnConnectedClientsClicked;

	/** Manages the session history widget */
	const TSharedRef<FLiveServerSessionHistoryController> SessionHistoryController;
	/** Manages the package viewer widget */
	const TSharedRef<FConcertSessionPackageViewerController> PackageViewerController;
};
