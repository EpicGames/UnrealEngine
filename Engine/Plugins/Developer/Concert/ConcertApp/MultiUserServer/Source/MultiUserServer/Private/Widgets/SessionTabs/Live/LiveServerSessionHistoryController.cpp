// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveServerSessionHistoryController.h"

#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"
#include "IConcertSession.h"
#include "Settings/MultiUserServerColumnVisibilitySettings.h"
#include "Templates/NonNullPointer.h"

namespace UE::MultiUserServer::Private
{
	static SSessionHistory::FArguments MakeLiveTabSessionHistoryArguments()
	{
		return SSessionHistory::FArguments()
			.ColumnVisibilitySnapshot(UMultiUserServerColumnVisibilitySettings::GetSettings()->GetLiveActivityBrowserColumnVisibility())
			.SaveColumnVisibilitySnapshot_Lambda([](const FColumnVisibilitySnapshot& Snapshot)
			{
				UMultiUserServerColumnVisibilitySettings::GetSettings()->SetLiveActivityBrowserColumnVisibility(Snapshot);
			});
	}
}

FLiveServerSessionHistoryController::FLiveServerSessionHistoryController(TSharedRef<IConcertServerSession> InInspectedSession, TSharedRef<IConcertSyncServer> InSyncServer)
	: FServerSessionHistoryControllerBase(InInspectedSession->GetId(), UE::MultiUserServer::Private::MakeLiveTabSessionHistoryArguments())
	, SyncServer(MoveTemp(InSyncServer))
{
	ReloadActivities();
	UMultiUserServerColumnVisibilitySettings::GetSettings()->OnLiveActivityBrowserColumnVisibility().AddRaw(this, &FLiveServerSessionHistoryController::OnActivityListColumnVisibilitySettingsUpdated);

	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(GetSessionId()))
	{
		(*Database)->OnActivityProduced().AddRaw(this, &FLiveServerSessionHistoryController::OnSessionProduced);
	}
}

FLiveServerSessionHistoryController::~FLiveServerSessionHistoryController()
{
	if (UMultiUserServerColumnVisibilitySettings* Settings = UMultiUserServerColumnVisibilitySettings::GetSettings(); IsValid(Settings))
	{
		Settings->OnLiveActivityBrowserColumnVisibility().RemoveAll(this);
	}
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(GetSessionId()))
	{
		(*Database)->OnActivityProduced().RemoveAll(this);
	}
}

TOptional<FConcertSyncSessionDatabaseNonNullPtr> FLiveServerSessionHistoryController::GetSessionDatabase(const FGuid& InSessionId) const
{
	return SyncServer->GetLiveSessionDatabase(InSessionId);
}

void FLiveServerSessionHistoryController::OnSessionProduced(const FConcertSyncActivity& ProducedActivity)
{
	ReloadActivities();
}

void FLiveServerSessionHistoryController::OnActivityListColumnVisibilitySettingsUpdated(const FColumnVisibilitySnapshot& NewValue)
{
	GetSessionHistory()->OnColumnVisibilitySettingsChanged(NewValue);
}
