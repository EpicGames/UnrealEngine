// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SessionTabs/ServerSessionHistoryControllerBase.h"

struct FColumnVisibilitySnapshot;

class FConcertSyncSessionDatabase;
class IConcertSyncServer;
class IConcertServerSession;

struct FConcertSyncActivity;

/** Manages SSessionHistory by using IConcertSyncServer::GetLiveSessionDatabase for retrieving the session database. */
class FLiveServerSessionHistoryController : public FServerSessionHistoryControllerBase
{
public:

	FLiveServerSessionHistoryController(TSharedRef<IConcertServerSession> InspectedSession, TSharedRef<IConcertSyncServer> SyncServer);
	virtual ~FLiveServerSessionHistoryController() override;
	
protected:

	//~ Begin FServerSessionHistoryControllerBase Interface
	virtual TOptional<FConcertSyncSessionDatabaseNonNullPtr> GetSessionDatabase(const FGuid& InSessionId) const override;
	//~ End FServerSessionHistoryControllerBase Interface

private:

	TSharedRef<IConcertSyncServer> SyncServer;
	
	void OnSessionProduced(const FConcertSyncActivity& ProducedActivity);
	void OnActivityListColumnVisibilitySettingsUpdated(const FColumnVisibilitySnapshot& NewValue);
};
