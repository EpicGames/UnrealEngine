// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSessionPackageViewerController.h"

#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"
#include "IConcertSyncServer.h"
#include "IConcertSession.h"
#include "Settings/MultiUserServerColumnVisibilitySettings.h"
#include "SConcertSessionPackageViewer.h"
#include "Templates/NonNullPointer.h"

FConcertSessionPackageViewerController::FConcertSessionPackageViewerController(TSharedRef<IConcertServerSession> InInspectedSession, TSharedRef<IConcertSyncServer> InSyncServer)
	: InspectedSession(MoveTemp(InInspectedSession))
	, SyncServer(MoveTemp(InSyncServer))
	, PackageViewer(MakePackageViewer())
{
	ReloadActivities();
	UMultiUserServerColumnVisibilitySettings::GetSettings()->OnLiveSessionContentColumnVisibility().AddRaw(this, &FConcertSessionPackageViewerController::OnSessionContentColumnVisibilitySettingsUpdated);
	
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(InspectedSession->GetId()))
	{
		(*Database)->OnActivityProduced().AddRaw(this, &FConcertSessionPackageViewerController::OnSessionProduced);
	}
}

FConcertSessionPackageViewerController::~FConcertSessionPackageViewerController()
{
	if (UMultiUserServerColumnVisibilitySettings* Settings = UMultiUserServerColumnVisibilitySettings::GetSettings(); IsValid(Settings))
	{
		Settings->OnLiveSessionContentColumnVisibility().RemoveAll(this);
	}
	
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(InspectedSession->GetId()))
	{
		(*Database)->OnActivityProduced().RemoveAll(this);
	}
}

void FConcertSessionPackageViewerController::ReloadActivities() const
{
	PackageViewer->ResetActivityList();
	
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(InspectedSession->GetId()))
	{
		struct FPackageActivity
		{
			FConcertSessionActivity Activity;
			FConcertSyncPackageEventData PackageData;
		};
		
		TMap<FName, FPackageActivity> LatestPackageActivities;
		(*Database)->EnumeratePackageActivities([this, &LatestPackageActivities](FConcertSyncActivity&& BasePart, FConcertSyncPackageEventData& EventData)
		{
			FStructOnScope ActivitySummary;
			if (BasePart.EventSummary.GetPayload(ActivitySummary))
			{
				FPackageActivity Activity{ { BasePart, ActivitySummary }, EventData };
				const bool bWasRenamed = Activity.PackageData.MetaData.PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Renamed;
				if (bWasRenamed)
				{
					LatestPackageActivities.Remove(EventData.MetaData.PackageInfo.PackageName);
					LatestPackageActivities.Emplace(EventData.MetaData.PackageInfo.NewPackageName, MoveTemp(Activity));
				}
				else
				{
					LatestPackageActivities.Emplace(EventData.MetaData.PackageInfo.PackageName, MoveTemp(Activity));
				}
			}
			return EBreakBehavior::Continue;
		});

		for (auto ActivityIt = LatestPackageActivities.CreateIterator(); ActivityIt; ++ActivityIt)
		{
			PackageViewer->AppendActivity(MoveTemp(ActivityIt->Value.Activity));
		}
	}
}

TSharedRef<SConcertSessionPackageViewer> FConcertSessionPackageViewerController::MakePackageViewer() const
{
	return SNew(SConcertSessionPackageViewer)
		.GetClientInfo_Raw(this, &FConcertSessionPackageViewerController::GetClientInfo)
		.GetPackageEvent_Raw(this, &FConcertSessionPackageViewerController::GetPackageEvent)
		.GetSizeOfPackageActivity_Raw(this, &FConcertSessionPackageViewerController::GetSizeOfPackageEvent);
}

TOptional<FConcertClientInfo> FConcertSessionPackageViewerController::GetClientInfo(FGuid ClientId) const
{
	const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(InspectedSession->GetId());
	FConcertSyncEndpointData Result;
	if (Database && (*Database)->GetEndpoint(ClientId, Result))
	{
		return Result.ClientInfo;
	}
	return {};
}

bool FConcertSessionPackageViewerController::GetPackageEvent(const FConcertSessionActivity& Activity, FConcertSyncPackageEventMetaData& OutPackageEvent) const
{
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(InspectedSession->GetId()))
	{
		return (*Database)->GetPackageEventMetaData(Activity.Activity.EventId, OutPackageEvent.PackageRevision, OutPackageEvent.PackageInfo);
	}
	return false;
}

TOptional<int64> FConcertSessionPackageViewerController::GetSizeOfPackageEvent(const FConcertSessionActivity& Activity) const
{
	const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = SyncServer->GetLiveSessionDatabase(InspectedSession->GetId());
	if (!Database)
	{
		return {};
	}

	const TSet<EConcertPackageUpdateType> PackageTypesWithSize { EConcertPackageUpdateType::Added, EConcertPackageUpdateType::Renamed, EConcertPackageUpdateType::Saved };
	FConcertSyncPackageEventMetaData PackageEventMetaData;
	if ((*Database)->GetPackageEventMetaData(Activity.Activity.EventId, PackageEventMetaData.PackageRevision, PackageEventMetaData.PackageInfo)
		; PackageTypesWithSize.Contains(PackageEventMetaData.PackageInfo.PackageUpdateType))
	{
		const FName PackageName = PackageEventMetaData.PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Renamed
			?  PackageEventMetaData.PackageInfo.NewPackageName
			: PackageEventMetaData.PackageInfo.PackageName;
		return (*Database)->GetPackageSizeForRevision(PackageName);
	}
	return {};
}

void FConcertSessionPackageViewerController::OnSessionProduced(const FConcertSyncActivity& ProducedActivity) const
{
	if (ProducedActivity.EventType == EConcertSyncActivityEventType::Package)
	{
		ReloadActivities();
	}
}

void FConcertSessionPackageViewerController::OnSessionContentColumnVisibilitySettingsUpdated(const FColumnVisibilitySnapshot& NewValue)
{
	GetPackageViewer()->OnColumnVisibilitySettingsChanged(NewValue);
}
