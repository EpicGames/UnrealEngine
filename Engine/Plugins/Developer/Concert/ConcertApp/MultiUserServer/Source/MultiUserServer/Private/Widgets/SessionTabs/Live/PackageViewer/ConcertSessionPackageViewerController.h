// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

template <typename OptionalType> struct TOptional;

class IConcertSyncServer;
class IConcertServerSession;
class SConcertSessionPackageViewer;

struct FConcertClientInfo;
struct FConcertSessionActivity;
struct FConcertSyncActivity;
struct FConcertSyncPackageEventMetaData;
struct FColumnVisibilitySnapshot;

/** Manages SConcertSessionPackageViewer for a server. */
class FConcertSessionPackageViewerController : public TSharedFromThis<FConcertSessionPackageViewerController>
{
public:

	FConcertSessionPackageViewerController(TSharedRef<IConcertServerSession> InspectedSession, TSharedRef<IConcertSyncServer> SyncServer);
	~FConcertSessionPackageViewerController();
	
	void ReloadActivities() const;
	TSharedRef<SConcertSessionPackageViewer> GetPackageViewer() const { return PackageViewer; }

private:
	
	TSharedRef<IConcertServerSession> InspectedSession;
	TSharedRef<IConcertSyncServer> SyncServer;
	
	/** The widget being managed */
	TSharedRef<SConcertSessionPackageViewer> PackageViewer;
	
	TSharedRef<SConcertSessionPackageViewer> MakePackageViewer() const;
	
	TOptional<FConcertClientInfo> GetClientInfo(FGuid ClientId) const;
	bool GetPackageEvent(const FConcertSessionActivity& Activity, FConcertSyncPackageEventMetaData& OutPackageEvent) const;
	TOptional<int64> GetSizeOfPackageEvent(const FConcertSessionActivity& Activity) const;
	void OnSessionProduced(const FConcertSyncActivity& ProducedActivity) const;
	
	void OnSessionContentColumnVisibilitySettingsUpdated(const FColumnVisibilitySnapshot& NewValue);
};
