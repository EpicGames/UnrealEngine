// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageViewerColumns.h"

#include "ConcertServerStyle.h"

#include "ConcertSyncSessionTypes.h"
#include "Math/UnitConversion.h"

#include "Session/Activity/ActivityColumn.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Session/Activity/SConcertSessionActivities.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"

const FName UE::MultiUserServer::PackageViewerColumns::PackageUpdateTypeColumnId	= TEXT("PackageUpdateTypeColumnId");
const FName UE::MultiUserServer::PackageViewerColumns::SizeColumnId					= TEXT("SizeColumnId");
const FName UE::MultiUserServer::PackageViewerColumns::VersionColumnId				= TEXT("VersionColumnId");

namespace UE::MultiUserServer::PackageViewerColumns
{
	using FFormatString = TFunction<FString(int64 Number)>;
	
	static FActivityColumn CreateNumericColumn(
		FName ColumnId,
		FText Label,
		int32 Width,
		EPredefinedPackageColumnOrder SortOrder,
		FGetNumericValueFromPackageActivity GetterFunc,
		FFormatString FormatFunc)
	{
		return FActivityColumn(
			SHeaderRow::Column(ColumnId)
				.DefaultLabel(Label)
				.FillSized(Width)
			)
			.ColumnSortOrder(static_cast<int32>(SortOrder))
			.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([GetterFunc, FormatFunc](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
			{
				Slot
				.HAlign(HAlign_Center)
				.Padding(1)
				[
					SNew(STextBlock)
					.Text_Lambda([GetterFunc, FormatFunc, WeakActivity = TWeakPtr<FConcertSessionActivity>(Activity)]()
					{
						const TSharedPtr<FConcertSessionActivity> PinnedActivity = WeakActivity.Pin();
						check(PinnedActivity);
						const TOptional<int64> Size = GetterFunc.Execute(*PinnedActivity.Get());
						return FText::FromString(Size ? FormatFunc(*Size) : TEXT("n/a"));
					})
					.HighlightText(Owner->GetHighlightText())
				];
			}))
			.PopulateSearchString(FActivityColumn::FPopulateSearchString::CreateLambda([GetterFunc, FormatFunc](const TSharedRef<SConcertSessionActivities>& Owner, const FConcertSessionActivity& Activity, TArray<FString>& OutSearchStrings)
			{
				const TOptional<int64> Size = GetterFunc.Execute(Activity);
				const FString Text = Size ? FormatFunc(*Size) : FString(TEXT("n/a"));
				OutSearchStrings.Add(Text);
			}));
	}
}

FActivityColumn UE::MultiUserServer::PackageViewerColumns::PackageUpdateTypeColumn(FGetPackageUpdateType GetPackageUpdateTypeFunc)
{
	return FActivityColumn(
		SHeaderRow::Column(PackageUpdateTypeColumnId)
			.HeaderContent()
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				[
					SNew(SImage)
					.Image(FConcertServerStyle::Get().GetBrush("Concert.SessionContent.ColumnHeader"))
				]
			]
			.FixedWidth(22)
			// Cannot be hidden
			.ShouldGenerateWidget(true)
		)
		.ColumnSortOrder(static_cast<int32>(EPredefinedPackageColumnOrder::PackageUpdateType))
		.GenerateColumnWidget(FActivityColumn::FGenerateColumnWidget::CreateLambda([GetPackageUpdateTypeFunc](const TSharedRef<SConcertSessionActivities>& Owner, const TSharedRef<FConcertSessionActivity>& Activity, SOverlay::FScopedWidgetSlotArguments& Slot)
		{
			if (const TOptional<EConcertPackageUpdateType> PackageUpdateType = GetPackageUpdateTypeFunc.Execute(Activity.Get()))
			{
				const TMap<EConcertPackageUpdateType, FName> PackageIcons {
					{ EConcertPackageUpdateType::Added, TEXT("Concert.SessionContent.PackageAdded") },
					{ EConcertPackageUpdateType::Deleted, TEXT("Concert.SessionContent.PackageDeleted") },
					{ EConcertPackageUpdateType::Renamed, TEXT("Concert.SessionContent.PackageRenamed") },
					{ EConcertPackageUpdateType::Saved, TEXT("Concert.SessionContent.PackageSaved") },
					
					// This one is not expected to be found but it's here for safety
					{ EConcertPackageUpdateType::Dummy, TEXT("Concert.SessionContent.ColumnHeader")}
				};
				const TMap<EConcertPackageUpdateType, FText> ToolTips {
					{ EConcertPackageUpdateType::Added, LOCTEXT("PackageUpdateTypeColumn.Tooltip.Added", "Added packaged") },
					{ EConcertPackageUpdateType::Deleted, LOCTEXT("PackageUpdateTypeColumn.Tooltip.Deleted", "Deleted package") },
					{ EConcertPackageUpdateType::Renamed, LOCTEXT("PackageUpdateTypeColumn.Tooltip.Renamed", "Renamed package") },
					{ EConcertPackageUpdateType::Saved, LOCTEXT("PackageUpdateTypeColumn.Tooltip.Saved", "Saved package") },
					
					// This one is not expected to be found but it's here for safety
					{ EConcertPackageUpdateType::Dummy, LOCTEXT("PackageUpdateTypeColumn.Tooltip.Dummy", "Dummy") }
				};
				static_assert(static_cast<int32>(EConcertPackageUpdateType::Count) == 5, "You must update this TMap when you update EConcertPackageUpdateType.");
				
				Slot
				.HAlign(HAlign_Center)
				.Padding(1)
				[
					SNew(SImage)
					.Image(FConcertServerStyle::Get().GetBrush(PackageIcons[*PackageUpdateType]))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.ToolTipText(ToolTips[*PackageUpdateType])
				];
			}
			else
			{
				checkNoEntry();
				Slot[SNullWidget::NullWidget];
			}
		}));
}

FActivityColumn UE::MultiUserServer::PackageViewerColumns::SizeColumn(FGetSizeOfPackageActivity GetEventDataFunc)
{
	return CreateNumericColumn(
		SizeColumnId,
		LOCTEXT("Size", "Size"),
		80,
		EPredefinedPackageColumnOrder::Size,
		GetEventDataFunc,
		[](int64 Number)
		{
			const FNumericUnit<int64> Unit = FUnitConversion::QuantizeUnitsToBestFit(Number, EUnit::Bytes);
			return FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));
		});
}

FActivityColumn UE::MultiUserServer::PackageViewerColumns::VersionColumn(FGetVersionOfPackageActivity GetEventDataFunc)
{
	return CreateNumericColumn(
		VersionColumnId,
		LOCTEXT("Version", "Version"),
		80,
		EPredefinedPackageColumnOrder::Version,
		GetEventDataFunc,
		[](int64 Number)
		{
			return FString::Printf(TEXT("%lld"), Number);
		});
}

#undef LOCTEXT_NAMESPACE 
