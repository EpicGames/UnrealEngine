// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Session/Activity/PredefinedActivityColumns.h"

enum class EConcertPackageUpdateType : uint8;
struct FConcertSessionActivity;
template <typename OptionalType> struct TOptional;

namespace UE::MultiUserServer::PackageViewerColumns
{
	DECLARE_DELEGATE_RetVal_OneParam(TOptional<EConcertPackageUpdateType>, FGetPackageUpdateType, const FConcertSessionActivity& /*Activity*/);
	DECLARE_DELEGATE_RetVal_OneParam(TOptional<int64>, FGetNumericValueFromPackageActivity, const FConcertSessionActivity& /*Activity*/);
	using FGetSizeOfPackageActivity = FGetNumericValueFromPackageActivity;
	using FGetVersionOfPackageActivity = FGetNumericValueFromPackageActivity;
	
	enum class EPredefinedPackageColumnOrder : int32
	{
		PackageUpdateType = static_cast<int32>(ConcertSharedSlate::ActivityColumn::EPredefinedColumnOrder::AvatarColor) - 1,
		Size = static_cast<int32>(ConcertSharedSlate::ActivityColumn::EPredefinedColumnOrder::ClientName) + 1,
		Version = Size + 1
	};

	extern const FName PackageUpdateTypeColumnId;
	extern const FName SizeColumnId;
	extern const FName VersionColumnId;

	FActivityColumn PackageUpdateTypeColumn(FGetPackageUpdateType GetPackageUpdateTypeFunc);
	FActivityColumn SizeColumn(FGetSizeOfPackageActivity GetEventDataFunc);
	FActivityColumn VersionColumn(FGetVersionOfPackageActivity GetEventDataFunc);
};
