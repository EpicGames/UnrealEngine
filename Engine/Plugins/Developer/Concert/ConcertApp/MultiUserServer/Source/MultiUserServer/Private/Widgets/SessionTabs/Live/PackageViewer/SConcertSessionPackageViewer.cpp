// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertSessionPackageViewer.h"

#include "ConcertSyncSessionTypes.h"
#include "Settings/MultiUserServerColumnVisibilitySettings.h"

#include "Session/Activity/ActivityColumn.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SessionTabs/Live/PackageViewer/PackageViewerColumns.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"

void SConcertSessionPackageViewer::Construct(const FArguments& InArgs)
{
	using namespace UE::ConcertSharedSlate;
	using namespace UE::MultiUserServer;

	check(InArgs._GetSizeOfPackageActivity.IsBound());
	
	ActivityListViewOptions = MakeShared<FConcertSessionActivitiesOptions>();
	ActivityListViewOptions->bEnableConnectionActivityFiltering = false;
	ActivityListViewOptions->bEnableLockActivityFiltering = false;
	ActivityListViewOptions->bEnablePackageActivityFiltering = false;
	ActivityListViewOptions->bEnableTransactionActivityFiltering = false;

	SAssignNew(ActivityListView, SConcertSessionActivities)
		.OnGetPackageEvent(InArgs._GetPackageEvent)
		.OnMapActivityToClient(InArgs._GetClientInfo)
		.HighlightText(this, &SConcertSessionPackageViewer::HighlightSearchedText)
		.Columns({
			PackageViewerColumns::PackageUpdateTypeColumn(PackageViewerColumns::FGetPackageUpdateType::CreateSP(this, &SConcertSessionPackageViewer::GetPackageActivityUpdateType, InArgs._GetPackageEvent)),
			PackageViewerColumns::SizeColumn(InArgs._GetSizeOfPackageActivity),
			PackageViewerColumns::VersionColumn(PackageViewerColumns::FGetVersionOfPackageActivity::CreateSP(this, &SConcertSessionPackageViewer::GetVersionOfPackageActivity, InArgs._GetPackageEvent))
		})
		.TimeFormat(ActivityListViewOptions.Get(), &FConcertSessionActivitiesOptions::GetTimeFormat)
		.ConnectionActivitiesVisibility(EVisibility::Collapsed)
		.LockActivitiesVisibility(EVisibility::Collapsed)
		.PackageActivitiesVisibility(EVisibility::Visible)
		.TransactionActivitiesVisibility(EVisibility::Collapsed)
		.ReplicationActivitiesVisibility(EVisibility::Collapsed)
		.DetailsAreaVisibility(EVisibility::Collapsed)
		.IsAutoScrollEnabled(true)
		.ColumnVisibilitySnapshot(UMultiUserServerColumnVisibilitySettings::GetSettings()->GetLiveSessionContentColumnVisibility())
		.SaveColumnVisibilitySnapshot_Lambda([](const FColumnVisibilitySnapshot& Snapshot)
		{
			UMultiUserServerColumnVisibilitySettings::GetSettings()->SetLiveSessionContentColumnVisibility(Snapshot);
		});

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(1, 1)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("SearchHint", "Search..."))
				.OnTextChanged(this, &SConcertSessionPackageViewer::OnSearchTextChanged)
				.OnTextCommitted(this, &SConcertSessionPackageViewer::OnSearchTextCommitted)
				.DelayChangeNotificationsWhileTyping(true)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				ActivityListViewOptions->MakeViewOptionsComboButton()
			]
		]

		+SVerticalBox::Slot()
		[
			ActivityListView.ToSharedRef()
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			ActivityListViewOptions->MakeStatusBar(
				TAttribute<int32>(ActivityListView.Get(), &SConcertSessionActivities::GetTotalActivityNum),
				TAttribute<int32>(ActivityListView.Get(), &SConcertSessionActivities::GetDisplayedActivityNum)
				)
		]
	];
}

void SConcertSessionPackageViewer::ResetActivityList()
{
	ActivityListView->ResetActivityList();
}

void SConcertSessionPackageViewer::AppendActivity(FConcertSessionActivity Activity)
{
	const TSharedRef<FConcertSessionActivity> NewActivity = MakeShared<FConcertSessionActivity>(MoveTemp(Activity));
	ActivityListView->Append(NewActivity);
}

void SConcertSessionPackageViewer::OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot)
{
	ActivityListView->OnColumnVisibilitySettingsChanged(ColumnSnapshot);
}

TOptional<EConcertPackageUpdateType> SConcertSessionPackageViewer::GetPackageActivityUpdateType(const FConcertSessionActivity& Activity, SConcertSessionActivities::FGetPackageEvent GetPackageEventFunc) const
{
	FConcertSyncPackageEventMetaData PackageEventMetaData;
	if (GetPackageEventFunc.Execute(Activity, PackageEventMetaData))
	{
		return PackageEventMetaData.PackageInfo.PackageUpdateType;
	}
	return {};
}

TOptional<int64> SConcertSessionPackageViewer::GetVersionOfPackageActivity(const FConcertSessionActivity& Activity, SConcertSessionActivities::FGetPackageEvent GetPackageEventFunc) const
{
	FConcertSyncPackageEventMetaData PackageEventMetaData;
	if (GetPackageEventFunc.Execute(Activity, PackageEventMetaData))
	{
		const TSet<EConcertPackageUpdateType> NoRevisionTypes { EConcertPackageUpdateType::Deleted, EConcertPackageUpdateType::Dummy };
		return NoRevisionTypes.Contains(PackageEventMetaData.PackageInfo.PackageUpdateType)
			? TOptional<int64>{}
			: PackageEventMetaData.PackageRevision;
	}
	return {};
}

void SConcertSessionPackageViewer::OnSearchTextChanged(const FText& InSearchText)
{
	SearchedText = InSearchText;
	SearchBox->SetError(ActivityListView->UpdateTextFilter(InSearchText));
}

void SConcertSessionPackageViewer::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType)
{
	if (!InFilterText.EqualTo(SearchedText))
	{
		OnSearchTextChanged(InFilterText);
	}
}

FText SConcertSessionPackageViewer::HighlightSearchedText() const
{
	return SearchedText;
}

#undef LOCTEXT_NAMESPACE
