// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PackageViewerColumns.h"

#include "Session/Activity/SConcertSessionActivities.h"

class SSearchBox;
struct FColumnVisibilitySnapshot;

class FConcertSessionActivitiesOptions;

/** Displays package activities that happened in a concert session. */
class SConcertSessionPackageViewer : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SConcertSessionPackageViewer) {}
		SLATE_EVENT(SConcertSessionActivities::FGetPackageEvent, GetPackageEvent)
		SLATE_EVENT(SConcertSessionActivities::FGetActivityClientInfoFunc, GetClientInfo)
		SLATE_EVENT(UE::MultiUserServer::PackageViewerColumns::FGetSizeOfPackageActivity, GetSizeOfPackageActivity)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void ResetActivityList();
	void AppendActivity(FConcertSessionActivity Activity);

	void OnColumnVisibilitySettingsChanged(const FColumnVisibilitySnapshot& ColumnSnapshot);

private:

	TSharedPtr<SConcertSessionActivities> ActivityListView;
	
	/** Controls the activity list view options */
	TSharedPtr<FConcertSessionActivitiesOptions> ActivityListViewOptions;

	/** The widget used to enter the text to search. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The searched text to highlight. */
	FText SearchedText;

	TOptional<EConcertPackageUpdateType> GetPackageActivityUpdateType(const FConcertSessionActivity& Activity, SConcertSessionActivities::FGetPackageEvent GetPackageEventFunc) const;
	TOptional<int64> GetVersionOfPackageActivity(const FConcertSessionActivity& Activity, SConcertSessionActivities::FGetPackageEvent GetPackageEventFunc) const;

	/** Invoked when the text in the search box widget changes. */
	void OnSearchTextChanged(const FText& InSearchText);
	/** Invoked when the text in the search box widget is committed. */
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType);
	/** Returns the text to highlight when the search bar has a text set. */
	FText HighlightSearchedText() const;
};
