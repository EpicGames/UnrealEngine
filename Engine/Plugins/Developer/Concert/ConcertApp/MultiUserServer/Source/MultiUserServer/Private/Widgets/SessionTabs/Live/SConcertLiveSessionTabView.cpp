// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertLiveSessionTabView.h"

#include "ConcertServerStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Session/History/SSessionHistory.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SessionTabs/Live/PackageViewer/SConcertSessionPackageViewer.h"

#define LOCTEXT_NAMESPACE "MultiUserServerUI.SConcertSessionInspector"

const FName SConcertLiveSessionTabView::HistoryTabId("HistoryTabId");
const FName SConcertLiveSessionTabView::SessionContentTabId("SessionContentTabId");

void SConcertLiveSessionTabView::Construct(const FArguments& InArgs, const FRequiredWidgets& InRequiredArgs, FName StatusBarId)
{
	SConcertTabViewWithManagerBase::Construct(
		SConcertTabViewWithManagerBase::FArguments()
		.ConstructUnderWindow(InRequiredArgs.ConstructUnderWindow)
		.ConstructUnderMajorTab(InRequiredArgs.ConstructUnderMajorTab)
		.CreateTabs(FCreateTabs::CreateLambda([this, &InRequiredArgs](const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout)
		{
			CreateTabs(InTabManager, InLayout, InRequiredArgs);
		}))
		.OverlayTabs_Lambda([this, &InArgs](const TSharedRef<SWidget>& Tabs)
		{
			return SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(3.f)
				[
					CreateToolbar(InArgs)
				]
			
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					Tabs
				];
		})
		.LayoutName("ConcertSessionInspector_v0.3"),
		StatusBarId
		);
}

TSharedRef<SWidget> SConcertLiveSessionTabView::CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FRequiredWidgets& InRequiredArgs)
{
	const TSharedRef<FWorkspaceItem> WorkspaceItem = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("ArchiveSession", "Live Session"));
	
	InTabManager->RegisterTabSpawner(HistoryTabId, FOnSpawnTab::CreateSP(this, &SConcertLiveSessionTabView::SpawnActivityHistory, InRequiredArgs.SessionHistory))
		.SetDisplayName(LOCTEXT("ActivityHistoryLabel", "History"))
		.SetGroup(WorkspaceItem);
	
	InTabManager->RegisterTabSpawner(SessionContentTabId, FOnSpawnTab::CreateSP(this, &SConcertLiveSessionTabView::SpawnSessionContent, InRequiredArgs.PackageViewer))
		.SetDisplayName(LOCTEXT("SessionContentLabel", "Session Content"))
		.SetGroup(WorkspaceItem)
		.SetIcon(FSlateIcon(FConcertServerStyle::GetStyleSetName(), TEXT("Concert.Icon.Package")));

	InLayout->AddArea
	(
		FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.5f)
				->AddTab(HistoryTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.5f)
				->AddTab(SessionContentTabId, ETabState::ClosedTab)
			)
	);

	return InTabManager->RestoreFrom(InLayout, InRequiredArgs.ConstructUnderWindow).ToSharedRef();
}

TSharedRef<SDockTab> SConcertLiveSessionTabView::SpawnActivityHistory(const FSpawnTabArgs& Args, TSharedRef<SSessionHistory> SessionHistory)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("ActivityHistoryLabel", "History"))
		.TabRole(PanelTab)
		[
			SessionHistory
		];
}

TSharedRef<SDockTab> SConcertLiveSessionTabView::SpawnSessionContent(const FSpawnTabArgs& Args, TSharedRef<SConcertSessionPackageViewer> PackageViewer)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("SessionContentLabel", "Session Content"))
		.TabRole(PanelTab)
		[
			PackageViewer
		];
}

TSharedRef<SWidget> SConcertLiveSessionTabView::CreateToolbar(const FArguments& InArgs)
{
	return SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("ConnectedClientsTooltip", "Opens Clients tab and shows all clients connected to this session."))
			.ContentPadding(FMargin(1, 0))
			.OnClicked_Lambda([Callback = InArgs._OnConnectedClientsClicked]()
			{
				Callback.ExecuteIfBound();
				return FReply::Handled();
			})
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FConcertServerStyle::Get().GetBrush("Concert.Icon.Client"))
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ConnectedClients", "Connected Clients"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
