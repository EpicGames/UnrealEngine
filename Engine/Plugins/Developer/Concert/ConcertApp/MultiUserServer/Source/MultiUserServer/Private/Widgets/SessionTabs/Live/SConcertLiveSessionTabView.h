// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SConcertTabViewWithManagerBase.h"

class SConcertSessionPackageViewer;
class SSessionHistory;

class FTabManager;
class FSpawnTabArgs;
class SDockTab;
class SWindow;

/**
 * Designed to be the content of a tab showing:
 *  - activity history (transactions stored on the server as well as who made those transactions)
 *  - session content (list of session data saved during a Multi-user session)
 *  - connection monitor (details about the connected clients on the given session and network info)
 * Implements view in model-view-controller pattern.
 */
class SConcertLiveSessionTabView : public SConcertTabViewWithManagerBase
{
public:

	struct FRequiredWidgets
	{
		TSharedRef<SDockTab> ConstructUnderMajorTab;
		TSharedRef<SWindow> ConstructUnderWindow;
		TSharedRef<SSessionHistory> SessionHistory;
		TSharedRef<SConcertSessionPackageViewer> PackageViewer;

		FRequiredWidgets(TSharedRef<SDockTab> ConstructUnderMajorTab,
				TSharedRef<SWindow> ConstructUnderWindow,
				TSharedRef<SSessionHistory> SessionHistoryController,
				TSharedRef<SConcertSessionPackageViewer> PackageViewerController)
			: ConstructUnderMajorTab(MoveTemp(ConstructUnderMajorTab))
			, ConstructUnderWindow(MoveTemp(ConstructUnderWindow))
			, SessionHistory(MoveTemp(SessionHistoryController))
			, PackageViewer(MoveTemp(PackageViewerController))
		{}
	};

	static const FName HistoryTabId;
	static const FName SessionContentTabId;

	SLATE_BEGIN_ARGS(SConcertLiveSessionTabView) {}
		SLATE_NAMED_SLOT(FArguments, StatusBar)
		SLATE_EVENT(FSimpleDelegate, OnConnectedClientsClicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FRequiredWidgets& InRequiredArgs, FName StatusBarId);

private:

	// Spawn tabs
	TSharedRef<SWidget> CreateTabs(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FLayout>& InLayout, const FRequiredWidgets& InRequiredArgs);
	TSharedRef<SDockTab> SpawnActivityHistory(const FSpawnTabArgs& Args, TSharedRef<SSessionHistory> SessionHistory);
	TSharedRef<SDockTab> SpawnSessionContent(const FSpawnTabArgs& Args, TSharedRef<SConcertSessionPackageViewer> PackageViewer);

	TSharedRef<SWidget> CreateToolbar(const FArguments& InArgs);
};
