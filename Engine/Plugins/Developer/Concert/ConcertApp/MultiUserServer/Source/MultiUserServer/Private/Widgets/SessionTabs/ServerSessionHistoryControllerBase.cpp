// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerSessionHistoryControllerBase.h"

#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"
#include "ServerUndoHistoryReflectionDataProvider.h"
#include "Templates/NonNullPointer.h"

FServerSessionHistoryControllerBase::FServerSessionHistoryControllerBase(FGuid SessionId, SSessionHistory::FArguments Arguments)
	: FAbstractSessionHistoryController(MoveTemp(Arguments.UndoHistoryReflectionProvider(MakeShared<UE::MultiUserServer::FServerUndoHistoryReflectionDataProvider>())))
	, SessionId(MoveTemp(SessionId))
{}

void FServerSessionHistoryControllerBase::GetActivities(int64 MaximumNumberOfActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, TArray<FConcertSessionActivity>& OutFetchedActivities) const
{
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = GetSessionDatabase(SessionId))
	{
		OutEndpointClientInfoMap.Reset();
		OutFetchedActivities.Reset();

		int64 LastActivityId = INDEX_NONE;
		(*Database)->GetActivityMaxId(LastActivityId);
		const int64 FirstActivityIdToFetch = FMath::Max<int64>(1, LastActivityId - MaximumNumberOfActivities);
		(*Database)->EnumerateActivitiesInRange(FirstActivityIdToFetch, MaximumNumberOfActivities, [this, &Database, &OutEndpointClientInfoMap, &OutFetchedActivities](FConcertSyncActivity&& InActivity)
		{
			if (!OutEndpointClientInfoMap.Contains(InActivity.EndpointId))
			{
				FConcertSyncEndpointData EndpointData;
				if ((*Database)->GetEndpoint(InActivity.EndpointId, EndpointData))
				{
					OutEndpointClientInfoMap.Add(InActivity.EndpointId, EndpointData.ClientInfo);
				}
			}

			FStructOnScope ActivitySummary;
			if (InActivity.EventSummary.GetPayload(ActivitySummary))
			{
				OutFetchedActivities.Emplace(MoveTemp(InActivity), MoveTemp(ActivitySummary));
			}

			return EBreakBehavior::Continue;
		});
	}
}

bool FServerSessionHistoryControllerBase::GetPackageEvent(const FConcertSessionActivity& Activity, FConcertSyncPackageEventMetaData& OutPackageEvent) const
{
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = GetSessionDatabase(SessionId))
	{
		return (*Database)->GetPackageEventMetaData(Activity.Activity.EventId, OutPackageEvent.PackageRevision, OutPackageEvent.PackageInfo);
	}
	
	return false;
}

TFuture<TOptional<FConcertSyncTransactionEvent>> FServerSessionHistoryControllerBase::GetTransactionEvent(const FConcertSessionActivity& Activity) const
{
	if (const TOptional<FConcertSyncSessionDatabaseNonNullPtr> Database = GetSessionDatabase(SessionId))
	{
		return FindOrRequestTransactionEvent(**Database, Activity.Activity.EventId);
	}
	
	return MakeFulfilledPromise<TOptional<FConcertSyncTransactionEvent>>().GetFuture(); // Not found.
}

TFuture<TOptional<FConcertSyncTransactionEvent>> FServerSessionHistoryControllerBase::FindOrRequestTransactionEvent(const FConcertSyncSessionDatabase& Database, const int64 TransactionEventId) const
{
	FConcertSyncTransactionEvent TransactionEvent;
	return Database.GetTransactionEvent(TransactionEventId, TransactionEvent, false)
		?	MakeFulfilledPromise<TOptional<FConcertSyncTransactionEvent>>(MoveTemp(TransactionEvent)).GetFuture()
		:	MakeFulfilledPromise<TOptional<FConcertSyncTransactionEvent>>().GetFuture();
}
