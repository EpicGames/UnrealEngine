// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "IConcertSyncServer.h"
#include "Session/History/AbstractSessionHistoryController.h"
#include "Session/History/SSessionHistory.h"

struct FConcertSyncActivity;
class FConcertSyncSessionDatabase;
class IConcertSyncServer;
class IConcertServerSession;

/**
 * Abstract base class providing most functionality for managing SSessionHistory on the server.
 * Subclasses control which session database is used: archived or live.
 */
class FServerSessionHistoryControllerBase : public FAbstractSessionHistoryController
{
public:

	FServerSessionHistoryControllerBase(FGuid SessionId, SSessionHistory::FArguments Arguments = SSessionHistory::FArguments());
	
protected:

	//~ Begin FAbstractSessionHistoryController Interface
	virtual void GetActivities(int64 MaximumNumberOfActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, TArray<FConcertSessionActivity>& OutFetchedActivities) const override;
	virtual bool GetPackageEvent(const FConcertSessionActivity& Activity, FConcertSyncPackageEventMetaData& OutPackageEvent) const override;
	virtual TFuture<TOptional<FConcertSyncTransactionEvent>> GetTransactionEvent(const FConcertSessionActivity& Activity) const override;
	//~ End FAbstractSessionHistoryController Interface

	/** Subclasses control which types of transaction are displayed: archived or live */
	virtual TOptional<FConcertSyncSessionDatabaseNonNullPtr> GetSessionDatabase(const FGuid& InSessionId) const = 0;

	const FGuid& GetSessionId() const { return SessionId; }
	
private:

	/** The ID of the managed session */
	FGuid SessionId;

	TFuture<TOptional<FConcertSyncTransactionEvent>> FindOrRequestTransactionEvent(const FConcertSyncSessionDatabase& Database, const int64 TransactionEventId) const;
};
