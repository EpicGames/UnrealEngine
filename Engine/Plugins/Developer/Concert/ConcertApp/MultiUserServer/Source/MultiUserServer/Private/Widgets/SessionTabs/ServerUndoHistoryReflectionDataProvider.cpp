// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerUndoHistoryReflectionDataProvider.h"

#include "Algo/AnyOf.h"

void UE::MultiUserServer::FServerUndoHistoryReflectionDataProvider::SetTransactionContext(const FConcertTransactionEventBase& InTransaction)
{
	TransactionContext = InTransaction;
}

bool UE::MultiUserServer::FServerUndoHistoryReflectionDataProvider::HasClassDisplayName(const FSoftClassPath& ClassPath) const
{
	return Algo::AnyOf(TransactionContext.ExportedObjects, [ClassPath](const FConcertExportedObject& Object)
	{
		return Object.ObjectId.ObjectClassPathName.ToString() == ClassPath.ToString();
	});
}

TOptional<FString> UE::MultiUserServer::FServerUndoHistoryReflectionDataProvider::GetClassDisplayName(const FSoftClassPath& ClassPath) const
{
	const FString AsString = ClassPath.ToString();
	int32 Index;
	if (AsString.FindLastChar(TEXT('/'), Index))
	{
		return AsString.RightChop(Index + 1);
	}

	return {};
}
