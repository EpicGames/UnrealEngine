// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertTransactionEvents.h"
#include "Session/Activity/IConcertReflectionDataProvider.h"

namespace UE::MultiUserServer
{
	class FServerUndoHistoryReflectionDataProvider : public ConcertSharedSlate::IConcertReflectionDataProvider
	{
	public:

		//~ Begin IConcertReflectionDataProvider Interface
		virtual void SetTransactionContext(const FConcertTransactionEventBase& InTransaction) override;
		//~ End IConcertReflectionDataProvider Interface

		//~ Begin IReflectionDataProvider Interface
		virtual bool HasClassDisplayName(const FSoftClassPath& ClassPath) const override;
		virtual TOptional<FString> GetClassDisplayName(const FSoftClassPath& ClassPath) const override;
		virtual bool SupportsGetPropertyReflectionData() const override { return false; }
		virtual TOptional<UndoHistory::FPropertyReflectionData> GetPropertyReflectionData(const FSoftClassPath& ClassPath, FName PropertyName) const override { return {}; }
		//~ End IReflectionDataProvider Interface

	private:

		FConcertTransactionEventBase TransactionContext;
	};
}


