// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertStatusBar.h"

#include "OutputLogCreationParams.h"

#include "Framework/Application/SlateApplication.h"
#include "OutputLogModule.h"
#include "SWidgetDrawer.h"
#include "WidgetDrawerConfig.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI"

namespace UE::ConcertServerUI::Private
{
	const FName OutputLogId("OutputLog");
	
	class FStatusBarSingleton
	{
		TSharedPtr<SWidget> StatusBarOutputLog;
		TArray<TWeakPtr<SWidgetDrawer>> StatusBars;
		
		TSharedRef<SWidget> OnGetOutputLog()
		{
			if (!StatusBarOutputLog)
			{
				FOutputLogCreationParams Params;
				Params.bCreateDockInLayoutButton = true;
				Params.SettingsMenuCreationFlags = EOutputLogSettingsMenuFlags::SkipClearOnPie
					| EOutputLogSettingsMenuFlags::SkipOpenSourceButton
					| EOutputLogSettingsMenuFlags::SkipEnableWordWrapping; // Checkbox relies on saving an editor config file and does not work correctly
				StatusBarOutputLog = FOutputLogModule::Get().MakeOutputLogWidget(Params);
			}

			return StatusBarOutputLog.ToSharedRef();
		}
		
		void OnOutputLogOpened(FName StatusBarWithDrawerName)
		{
			// Dismiss all other open drawers - StatusBarOutputLog is shared and shouldn't be in the layout twice
			for (TWeakPtr<SWidgetDrawer> WidgetDrawer : StatusBars)
			{
				if (WidgetDrawer.IsValid())
				{
					TSharedPtr<SWidgetDrawer> PinnedDrawer = WidgetDrawer.Pin();
					if (StatusBarWithDrawerName != PinnedDrawer->GetDrawerName() || PinnedDrawer->IsAnyOtherDrawerOpened(OutputLogId))
					{
						PinnedDrawer->CloseDrawerImmediately();
					}
				}
			}
			
			FOutputLogModule::Get().FocusOutputLogConsoleBox(StatusBarOutputLog.ToSharedRef());
		}
		
		void OnOutputLogDismised(const TSharedPtr<SWidget>& NewlyFocusedWidget)
		{}

		void PreShutdownSlate()
		{
			StatusBarOutputLog.Reset();
		}

	public:

		void Init(TSharedRef<SWidgetDrawer> WidgetDrawer, FWidgetDrawerConfig& OutputLogDrawer)
		{
			if (!FSlateApplication::Get().OnPreShutdown().IsBoundToObject(this))
			{
				// Destroying StatusBarOutputLog in ~FStatusBarSingleton is too late: it causes a crash
				FSlateApplication::Get().OnPreShutdown().AddRaw(this, &FStatusBarSingleton::PreShutdownSlate);
			}

			const bool bIsDrawerNameUnique = !StatusBars.ContainsByPredicate([&WidgetDrawer](TWeakPtr<SWidgetDrawer> WeakDrawer)
			{
				return ensure(WeakDrawer.IsValid())
					&& WeakDrawer.Pin()->GetDrawerName() == WidgetDrawer->GetDrawerName();
			});
			checkf(bIsDrawerNameUnique, TEXT("Every widget drawer is expected to have an unique ID"));
			
			StatusBars.Add(MoveTemp(WidgetDrawer));
			
			OutputLogDrawer.GetDrawerContentDelegate.BindRaw(this, &FStatusBarSingleton::OnGetOutputLog);
			OutputLogDrawer.OnDrawerOpenedDelegate.BindRaw(this, &FStatusBarSingleton::OnOutputLogOpened);
			OutputLogDrawer.OnDrawerDismissedDelegate.BindRaw(this, &FStatusBarSingleton::OnOutputLogDismised);
		}

		void Remove(TSharedRef<SWidgetDrawer> WidgetDrawer)
		{
			StatusBars.RemoveSingle(WidgetDrawer);
		}
		
	} GStatusBarManager;
}

SConcertStatusBar::~SConcertStatusBar()
{
	UE::ConcertServerUI::Private::GStatusBarManager.Remove(WidgetDrawer.ToSharedRef());
}

void SConcertStatusBar::Construct(const FArguments& InArgs, FName StatusBarId)
{
	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(FAppStyle::Get().GetFloat("StatusBar.Height"))
		[
			MakeWidgetDrawer(StatusBarId)
		]
	];
}

TSharedRef<SWidgetDrawer> SConcertStatusBar::MakeWidgetDrawer(FName StatusBarId)
{
	using namespace UE::ConcertServerUI::Private;
	
	WidgetDrawer = SNew(SWidgetDrawer, StatusBarId);

	TSharedPtr<SMultiLineEditableTextBox> ConsoleEditBox;
	FSimpleDelegate OnConsoleClosed;
	FSimpleDelegate OnConsoleCommandExecuted;
	const TSharedRef<SWidget> OutputLog = 
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.0f, 0.0f))
			[
				SNew(SBox)
				.WidthOverride(350.f)
				[
					FOutputLogModule::Get().MakeConsoleInputBox(ConsoleEditBox, OnConsoleClosed, OnConsoleCommandExecuted)
				]
			];
	
	FWidgetDrawerConfig OutputLogDrawer(OutputLogId);
	GStatusBarManager.Init(WidgetDrawer.ToSharedRef(), OutputLogDrawer);
	OutputLogDrawer.CustomWidget = OutputLog;

	OutputLogDrawer.ButtonText = LOCTEXT("StatusBar_OutputLogButton", "Output Log");
	OutputLogDrawer.Icon = FAppStyle::Get().GetBrush("Log.TabIcon");
	WidgetDrawer->RegisterDrawer(MoveTemp(OutputLogDrawer));

	return WidgetDrawer.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
