// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SWidgetDrawer;

class SConcertStatusBar : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SConcertStatusBar) { }
	SLATE_END_ARGS()
	
	~SConcertStatusBar();

	void Construct(const FArguments& InArgs, FName StatusBarId);

private:

	TSharedPtr<SWidgetDrawer> WidgetDrawer;

	TSharedRef<SWidgetDrawer> MakeWidgetDrawer(FName StatusBarId);
};
