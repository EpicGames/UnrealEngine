// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertFilterUtils.h"

#include "Widgets/Layout/SSpacer.h"

namespace UE::ConcertFilterUtils
{
	/** Used for widgets you add using FMenuBuilder::AddSubMenu or FMenuBuilder::AddWidget. This makes sure that the widgets have the same width (for consistent visuals). */
	TSharedRef<SWidget> SetMenuWidgetWidth(const TSharedRef<SWidget>& Widget, bool bNeedsPaddingFromSubmenuRightArrow, const float ForcedWidth, const float SubMenuIconRightPadding)
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SSpacer)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, static_cast<int32>(bNeedsPaddingFromSubmenuRightArrow) * SubMenuIconRightPadding, 0.f)
			[
				SNew(SBox)
				.WidthOverride(ForcedWidth)
				[
					Widget
				]
			];
	}
}
