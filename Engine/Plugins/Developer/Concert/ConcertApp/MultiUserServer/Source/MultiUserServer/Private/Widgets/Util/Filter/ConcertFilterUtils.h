// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"

namespace UE::ConcertFilterUtils
{
	/** Used for widgets you add using FMenuBuilder::AddSubMenu or FMenuBuilder::AddWidget. This makes sure that the widgets have the same width (for consistent visuals). */
	TSharedRef<SWidget> SetMenuWidgetWidth(const TSharedRef<SWidget>& Widget, bool bNeedsPaddingFromSubmenuRightArrow = true, const float ForcedWidth = 100.f, const float SubMenuIconRightPadding = 15.f);

	/** @param TSelectableItem Must be (efficiently) copyable */
	template<typename TSelectableItem>
	struct TRadialMenuBuilder
	{
		DECLARE_DELEGATE_RetVal_OneParam(FText, FItemToText, const TSelectableItem& Item);
		DECLARE_DELEGATE_OneParam(FOnSelectItem, const TSelectableItem& Item);

		/**
		 * Adds a menu entry from which the user can choose an item from a list.
		 * 
		 * This function acts as abstraction: UX is evolving and there are two options considered 1. (What we're doing now) create a sub menu. 2. Create a combo button widget.
		 * Regardless of what we choose in the future, this function abstracts the above so only this function needs to be adjusted if we change the UX - instead of changing the call sites.
		 */
		static void AddRadialSubMenu(
			FMenuBuilder& MenuBuilder,
			TAttribute<FText> LabelAttribute,
			TAttribute<TSelectableItem> SelectedItemAttribute,
			TAttribute<TArray<TSelectableItem>> ItemsAttribute,
			FItemToText DisplayNameDelegate,
			FOnSelectItem OnSelectItemDelegate,
			const float ForcedWidth = 100.f)
		{
			const TSharedRef<SWidget> Widget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(MoveTemp(LabelAttribute))
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SetMenuWidgetWidth
					(
						SNew(SComboButton)
						.HasDownArrow(false)
						.ContentPadding(FMargin(0.f, 2.f)) // So it is as high as other widgets
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text_Lambda([SelectedItemAttribute, DisplayNameDelegate](){ return DisplayNameDelegate.Execute(SelectedItemAttribute.Get()); })
						]
					, false, ForcedWidth)
				];
		
			MenuBuilder.AddSubMenu(
				Widget,
				FNewMenuDelegate::CreateLambda(
				[ItemsAttribute, SelectedItemAttribute, DisplayNameDelegate, OnSelectItemDelegate](FMenuBuilder& MenuBuilder)
				{
					for (const TSelectableItem& Item : ItemsAttribute.Get())
					{
						MenuBuilder.AddMenuEntry(
							DisplayNameDelegate.Execute(Item),
							FText::GetEmpty(),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([OnSelectItemDelegate, Item](){
									OnSelectItemDelegate.Execute(Item);
								}),
								FCanExecuteAction::CreateLambda([] { return true; }),
								FIsActionChecked::CreateLambda([SelectedItemAttribute, Item](){ return Item == SelectedItemAttribute.Get(); })),
							NAME_None,
							EUserInterfaceActionType::RadioButton
						);
					}
				}),
				false,
				false
				);
		}
	};
	
}
