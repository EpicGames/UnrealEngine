// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Filters/FilterBase.h"

class FMenuBuilder;
class SWidget;

namespace UE::MultiUserServer
{
	/** A filter that is intended to be displayed in the UI. Every filter has one widget displaying it. */
	template<typename TFilterType>
	class TConcertFrontendFilter
		:
		public FFilterBase<TFilterType>,
		public TSharedFromThis<TConcertFrontendFilter<TFilterType>>
	{
	public:

		TConcertFrontendFilter(TSharedPtr<FFilterCategory> InCategory)
			: FFilterBase<TFilterType>(MoveTemp(InCategory))
		{}

		/** Exposes widgets with which the filter can be edited. */
		virtual void ExposeEditWidgets(FMenuBuilder& MenuBuilder) {}
		
		virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("EditFilter", NSLOCTEXT("TConcertFrontendFilter", "EditFilter", "Edit Filter"));
			ExposeEditWidgets(MenuBuilder);
		}

		virtual FText GetToolTipText() const { return FText::GetEmpty(); }
		virtual FLinearColor GetColor() const { return FLinearColor::White; }
		virtual FName GetIconName() const { return NAME_None; }
		virtual bool IsInverseFilter() const { return false; }
		virtual void ActiveStateChanged(bool bActive) {}
		virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const {}
		virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) {}
	};

	/**
	 * Helper class to implement filters
	 *
	 * Intended pattern:
	 *	1. Subclass TConcertFilter<TFilterType> and implement filter logic in TConcertFilterImpl, e.g. text search. This will act as a "model" in MVC.
	 *  2. Subclass TConcertFrontendLogFilterAggregate and handle creating UI in it. This will act as a "View" in MVC.
	 */
	template<typename TConcertFilterImpl, typename TFilterType, typename TWidgetType = SWidget>
	class TConcertFrontendFilterAggregate : public TConcertFrontendFilter<TFilterType>
	{
		using Super = TConcertFrontendFilter<TFilterType>;
	public:

		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(TFilterType InItem) const final override { return Implementation.PassesFilter(InItem); }
		//~ End FConcertLogFilter Interface
	
	protected:

		template<typename... TArg>
		TConcertFrontendFilterAggregate(TSharedPtr<FFilterCategory> InCategory, TArg&&... Arg)
			: Super(MoveTemp(InCategory))
			, Implementation(Forward<TArg>(Arg)...)
		{
			Implementation.OnChanged().AddLambda([this]()
			{
				Super::BroadcastChangedEvent();
			});
		}

		TConcertFilterImpl Implementation;
	};
}


