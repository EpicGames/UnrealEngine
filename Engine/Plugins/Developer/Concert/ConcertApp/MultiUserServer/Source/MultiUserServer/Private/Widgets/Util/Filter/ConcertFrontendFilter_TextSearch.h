// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertFrontendFilter.h"
#include "Filters/SFilterSearchBox.h"
#include "Misc/TextFilter.h"

class FFilterCategory;

class FConcertLogTokenizer;

namespace UE::MultiUserServer
{
	/** Allows advanced search by text. Implements Adapter pattern to wrap TTextFilter. */
	template<typename TFilterType>
	class TConcertFilter_TextSearch : public IFilter<TFilterType>
	{
	public:

		TConcertFilter_TextSearch()
			: TextFilter(MakeTextFilter())
		{
			TextFilter->OnChanged().AddLambda([this]
			{
				OnChangedEvent.Broadcast();
			});
		}

		TSharedRef<TTextFilter<TFilterType>> MakeTextFilter() const
		{
			return MakeShared<TTextFilter<TFilterType>>(TTextFilter<TFilterType>::FItemToStringArray::CreateRaw(this, &TConcertFilter_TextSearch<TFilterType>::GenerateSearchTerms));
		}
		
		//~ Begin FConcertLogFilter Interface
		virtual bool PassesFilter(TFilterType InItem) const override { return TextFilter->PassesFilter(InItem); }
		virtual typename IFilter<TFilterType>::FChangedEvent& OnChanged() override { return OnChangedEvent; }
		//~ End FConcertLogFilter Interface
	
		void SetRawFilterText(const FText& InFilterText) { TextFilter->SetRawFilterText(InFilterText); }
		TSharedRef<TTextFilter<TFilterType>> GetTextFilter() const { return TextFilter; }

	protected:
		
		/** Parses InItem into a bunch of strings that can be searched */
		virtual void GenerateSearchTerms(TFilterType InItem, TArray<FString>& OutTerms) const = 0;
		
	private:

		/** Does the actual string search */
		TSharedRef<TTextFilter<TFilterType>> TextFilter;
		
		typename IFilter<TFilterType>::FChangedEvent OnChangedEvent;
	};

	/** Creates a search bar */
	template<typename TTextSearchFilterType /* Expected subtype of TConcertFilter_TextSearch */, typename TFilterType>
	class TConcertFrontendFilter_TextSearch : public TConcertFrontendFilterAggregate<TTextSearchFilterType, TFilterType, SFilterSearchBox>
	{
		using Super = TConcertFrontendFilterAggregate<TTextSearchFilterType, TFilterType, SFilterSearchBox>;
	public:

		template<typename... TArg>
		TConcertFrontendFilter_TextSearch(TSharedPtr<FFilterCategory> InCategory, TArg&&... Arg)
			: Super(MoveTemp(InCategory), Forward<TArg>(Arg)...)
		{
			SAssignNew(SearchBox, SFilterSearchBox)
				.DelayChangeNotificationsWhileTyping(true)
				.OnTextChanged_Lambda([this](const FText& NewSearchText)
				{
					SearchText = NewSearchText;
					Super::Implementation.SetRawFilterText(NewSearchText);
				});
		}
		
		virtual FString GetName() const override { return FString("Text Search"); }
		virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealMultiUserUI.TConcertFrontendFilter_TextSearch", "DisplayLabel", "Text Search"); }
		
		const FText& GetSearchText() const { return SearchText; }
		TSharedRef<SFilterSearchBox> GetSearchBox() const { return SearchBox.ToSharedRef(); }
		TSharedRef<TTextFilter<TFilterType>> MakeTextFilter() const { return Super::Implementation.MakeTextFilter(); }

	private:

		FText SearchText;
		TSharedPtr<SFilterSearchBox> SearchBox; 
	};
}


