// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SConcertFilterBar.h"

#include "Algo/AllOf.h"

class FFilterCategory;
template <typename FilterType> class FCustomTextFilter;

namespace UE::MultiUserServer
{
	struct FFilterWidgetArgs
	{
		TSharedPtr<SWidget> _RightOfSearchBar;
		FMargin _RightOfSearchBarPadding{ 4.f, 0.f, 4.f, 0.f };

		FFilterWidgetArgs& RightOfSearchBar(TSharedPtr<SWidget> Widget) { _RightOfSearchBar = Widget; return *this; }
		FFilterWidgetArgs& RightOfSearchBarPadding(FMargin Widget) { _RightOfSearchBarPadding = Widget; return *this; }
	};
	
	/** A filter that contains multiple UI filters */
	template<typename TFilterType, typename TTextSearchFilterType>
	class TConcertFrontendRootFilter 
		:
		public IFilter<TFilterType>,
		public TSharedFromThis<TConcertFrontendRootFilter<TFilterType, TTextSearchFilterType>>
	{
	public:

		TConcertFrontendRootFilter(
			TSharedRef<TTextSearchFilterType> InTextSearchFilter,
			TArray<TSharedRef<TConcertFrontendFilter<TFilterType>>> InFrontendFilters,
			TArray<TSharedRef<IFilter<TFilterType>>> InNonVisualFilters = {},
			TSharedRef<FFilterCategory> InDefaultCategory = MakeShared<FFilterCategory>(NSLOCTEXT("UnrealMultiUserServer.TConcertFrontendRootFilter", "DefaultCategoryLabel", "Default"), FText::GetEmpty())
			)
			: DefaultCategory(MoveTemp(InDefaultCategory))
			, TextSearchFilter(MoveTemp(InTextSearchFilter))
			, NonVisualFilters(MoveTemp(InNonVisualFilters))
		{
			FilterBar = SNew(SConcertFilterBar<TFilterType>)
				.FilterSearchBox(TextSearchFilter->GetSearchBox())
				.CreateTextFilter(SBasicFilterBar<TFilterType>::FCreateTextFilter::CreateLambda([this]()
				{
					return MakeShared<FCustomTextFilter<TFilterType>>(TextSearchFilter->MakeTextFilter());
				}))
				.CustomFilters(InFrontendFilters)
				.OnFilterChanged_Raw(this, &TConcertFrontendRootFilter<TFilterType, TTextSearchFilterType>::BroadcastOnChanged);

			TextSearchFilter->OnChanged().AddRaw(this, &TConcertFrontendRootFilter<TFilterType, TTextSearchFilterType>::BroadcastOnChanged);
			for (const TSharedRef<IFilter<TFilterType>>& Filter : InFrontendFilters)
			{
				Filter->OnChanged().AddRaw(this, &TConcertFrontendRootFilter<TFilterType, TTextSearchFilterType>::BroadcastOnChanged);
			}
			for (const TSharedRef<IFilter<TFilterType>>& Filter : NonVisualFilters)
			{
				Filter->OnChanged().AddRaw(this, &TConcertFrontendRootFilter<TFilterType, TTextSearchFilterType>::BroadcastOnChanged);
			}
		}
		
		/** Builds the widget view for all contained filters */
		TSharedRef<SWidget> BuildFilterWidgets(const FFilterWidgetArgs& InArgs = {})
		{
			TSharedPtr<SHorizontalBox> TopRow;
			const TSharedRef<SVerticalBox> Result = SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(TopRow, SHorizontalBox)

					// Filter button
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SBasicFilterBar<TFilterType>::MakeAddFilterButton(FilterBar.ToSharedRef())
					]

					// Search bar
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.Padding(0, 2)
					[
						TextSearchFilter->GetSearchBox()
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					FilterBar.ToSharedRef()
				];

			if (InArgs._RightOfSearchBar)
			{
				TopRow->AddSlot()
				.AutoWidth()
				.Padding(InArgs._RightOfSearchBarPadding)
				[
					InArgs._RightOfSearchBar.ToSharedRef()
				];
			}
			
			return Result;
		}
	
		//~ Begin IFilter Interface
		virtual bool PassesFilter(TFilterType InItem) const override
		{
			const bool bPassesActiveFilters = FilterBar->GetAllActiveFilters()->PassesAllFilters(InItem);
			const bool bPassesTextSearch = TextSearchFilter->PassesFilter(InItem);
			const bool bPassesNonVisualFitlers = Algo::AllOf(
				NonVisualFilters,
				[&InItem](const TSharedRef<IFilter<TFilterType>>& AndFilter){ return AndFilter->PassesFilter(InItem); }
				);

			return bPassesActiveFilters && bPassesTextSearch && bPassesNonVisualFitlers;
		}
		virtual typename IFilter<TFilterType>::FChangedEvent& OnChanged() override { return OnChangedEvent; }
		//~ End IFilter Interface

		FORCEINLINE const TSharedRef<TTextSearchFilterType>& GetTextSearchFilter() const { return TextSearchFilter; }
		FORCEINLINE TSharedRef<FFilterCategory> GetDefaultCategory() const { return DefaultCategory; }
		
	private:

		/** The default category that filters are added to */
		TSharedRef<FFilterCategory> DefaultCategory;
		/** Displays the currently active filters */
		TSharedPtr<SConcertFilterBar<TFilterType>> FilterBar;
		
		/** The text search filter. Also in AllFilters. Separate variable to build search bar in new line. */
		TSharedRef<TTextSearchFilterType> TextSearchFilter;
	
		/** Filters that are combined using logical AND. */
		TArray<TSharedRef<IFilter<TFilterType>>> NonVisualFilters;

		void BroadcastOnChanged()
		{
			OnChangedEvent.Broadcast();
		}
		
		typename IFilter<TFilterType>::FChangedEvent OnChangedEvent;
	};
}


