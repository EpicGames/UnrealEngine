// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertLogGlobal.h"
#include "Filters/SBasicFilterBar.h"
#include "SConcertFilterBar.generated.h"

class SFilterSearchBox;
namespace UE::MultiUserServer { template <typename TFilterType> class TConcertFrontendFilter; }
template <typename FilterType> class FFilterBase;

UCLASS()
class UConcertFilterBarContext : public UObject
{
	GENERATED_BODY()
public:

	DECLARE_DELEGATE_OneParam(FOnPopulateMenu, UToolMenu*);
	FOnPopulateMenu PopulateMenu;
};

namespace UE::MultiUserServer
{
	template<typename TFilterType>
	class SConcertFilterBar : public SBasicFilterBar<TFilterType>
	{
		using Super = SBasicFilterBar<TFilterType>;
	public:

		using FOnFilterChanged = typename SBasicFilterBar<TFilterType>::FOnFilterChanged;
		using FCreateTextFilter = typename SBasicFilterBar<TFilterType>::FCreateTextFilter;

		SLATE_BEGIN_ARGS(SConcertFilterBar)
		{}

			/** Delegate for when filters have changed */
			SLATE_EVENT(SConcertFilterBar<TFilterType>::FOnFilterChanged, OnFilterChanged)
		
			/** Initial List of Custom Filters that will be added to the AddFilter Menu */
			SLATE_ARGUMENT(TArray<TSharedRef<TConcertFrontendFilter<TFilterType>>>, CustomFilters)
		
			/** A delegate to create a Text Filter for FilterType items. If provided, will allow creation of custom
			 * text filters from the filter dropdown menu.
			 * @see FCustomTextFilter
			 */
			SLATE_ARGUMENT(FCreateTextFilter, CreateTextFilter)
		
			/** An SFilterSearchBox that can be attached to this filter bar. When provided along with a CreateTextFilter
			 *  delegate, allows the user to save searches from the Search Box as text filters for the filter bar.
			 *	NOTE: Will bind a delegate to SFilterSearchBox::OnSaveSearchClicked
			 */
			SLATE_ARGUMENT(TSharedPtr<SFilterSearchBox>, FilterSearchBox)
		
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			TArray<TSharedRef<FFilterBase<TFilterType>>> TransformedCustomFilters;
			Algo::Transform(InArgs._CustomFilters, TransformedCustomFilters, [](const TSharedRef<TConcertFrontendFilter<TFilterType>>& Item){ return Item; });
			
			typename SBasicFilterBar<TFilterType>::FArguments Args;
			Args._OnFilterChanged = InArgs._OnFilterChanged;
			Args._CustomFilters = MoveTemp(TransformedCustomFilters);
			Args._CreateTextFilter = InArgs._CreateTextFilter;
			Args._FilterSearchBox = InArgs._FilterSearchBox;
		
			SBasicFilterBar<TFilterType>::Construct(Args.FilterPillStyle(EFilterPillStyle::Basic));
		}

	private:
		
		virtual TSharedRef<SWidget> MakeAddFilterMenu() override
		{
			const FName FilterMenuName = "ConcertFilterBar.ConcertFilterBar";
			if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
			{
				UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
				Menu->bShouldCloseWindowAfterMenuSelection = true;
				Menu->bCloseSelfOnly = true;

				Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
				{
					if (const UConcertFilterBarContext* Context = InMenu->FindContext<UConcertFilterBarContext>())
					{
						Context->PopulateMenu.Execute(InMenu);
					}
				}));
			}

			UConcertFilterBarContext* Context = NewObject<UConcertFilterBarContext>();
			Context->PopulateMenu = UConcertFilterBarContext::FOnPopulateMenu::CreateSP(this, &SConcertFilterBar::HandlePopulateAddFilterMenu);
			
			const FToolMenuContext ToolMenuContext(Context);
			return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
		}

		void HandlePopulateAddFilterMenu(UToolMenu* InMenu)
		{
			Super::PopulateCommonFilterSections(InMenu);

			UE_CLOGF(this->AllFilterCategories.Num() > 1, LogConcert, Warning, "This widget is designed for uncategorisied filters");
			FToolMenuSection& Section = InMenu->AddSection("BasicFilterBarFiltersMenu");
			for (const TSharedRef<FFilterBase<TFilterType>>& Filter : this->AllFrontendFilters)
			{
				const TSharedRef<TConcertFrontendFilter<TFilterType>> CastFilter = StaticCastSharedRef<TConcertFrontendFilter<TFilterType>>(Filter);
				Section.AddSubMenu(
					NAME_None,
					Filter->GetDisplayName(),
					Filter->GetToolTipText(),
					FNewMenuDelegate::CreateLambda([WeakFilter = TWeakPtr<TConcertFrontendFilter<TFilterType>>(CastFilter)](FMenuBuilder& MenuBuilder)
					{
						const TSharedPtr<TConcertFrontendFilter<TFilterType>> PinnedFilter = WeakFilter.Pin();
						if (ensure(PinnedFilter))
						{
							PinnedFilter->ExposeEditWidgets(MenuBuilder);
						}
					}),
					FUIAction(
						FExecuteAction::CreateLambda([this, WeakFilter = TWeakPtr<TConcertFrontendFilter<TFilterType>>(CastFilter)]()
						{
							const TSharedPtr<TConcertFrontendFilter<TFilterType>> PinnedFilter = WeakFilter.Pin();
							if (ensure(PinnedFilter))
							{
								Super::FrontendFilterClicked(PinnedFilter.ToSharedRef());
							}
						}),
						FCanExecuteAction(),
						FGetActionCheckState::CreateLambda([this, WeakFilter = TWeakPtr<TConcertFrontendFilter<TFilterType>>(CastFilter)]()
						{
							const TSharedPtr<TConcertFrontendFilter<TFilterType>> PinnedFilter = WeakFilter.Pin();
							if (ensure(PinnedFilter))
							{
								return Super::IsFilterActive(PinnedFilter)
									? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
							return ECheckBoxState::Undetermined;
						})),
					EUserInterfaceActionType::ToggleButton
				);
			}
		}
	};
}

