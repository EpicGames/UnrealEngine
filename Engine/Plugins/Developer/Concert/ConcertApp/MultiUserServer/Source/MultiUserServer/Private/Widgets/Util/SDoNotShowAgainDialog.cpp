// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDoNotShowAgainDialog.h"

#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SDoNotShowAgainDialog"

void UE::MultiUserServer::SDoNotShowAgainDialog::Construct(const FArguments& InArgs)
{
	SCustomDialog::Construct(
		SCustomDialog::FArguments()
		.Title(InArgs._Title)
		.Buttons(InArgs._Buttons)
		.Content()
		[
			SNew(SVerticalBox)
			
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.FillHeight(1.f)
			.Padding(0, 5, 0, 10)
			[
				InArgs._Content.Widget
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.Padding(0, 5, 0, 10)
			[
				SNew(SHorizontalBox)
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(DoNotShowAgainCheckbox, SCheckBox)
				]
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DoNotShowAgain", "Do not show again"))
				]
				
			]
		]
		.OnClosed_Lambda([this, Callback = InArgs._DoNotShowAgainCallback]()
		{
			Callback.Execute(DoNotShowAgainCheckbox->IsChecked());
		})
	);
}

#undef LOCTEXT_NAMESPACE
