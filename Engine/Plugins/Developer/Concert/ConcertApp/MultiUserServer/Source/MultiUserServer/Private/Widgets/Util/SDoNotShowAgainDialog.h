// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dialog/SCustomDialog.h"

class IConcertSyncServer;
class SCheckBox;

namespace UE::MultiUserServer
{
	/** Text message with a "Do not show again" checkbox. */
	class SDoNotShowAgainDialog : public SCustomDialog
	{
	public:

		DECLARE_DELEGATE_OneParam(FOnClosed, bool /*bDoNotShowAgain*/);

		SLATE_BEGIN_ARGS(SDoNotShowAgainDialog)
		{}
			SLATE_ARGUMENT(FText, Title)
			SLATE_ARGUMENT(TArray<FButton>, Buttons)
			SLATE_NAMED_SLOT(FArguments, Content)
			SLATE_EVENT(FOnClosed, DoNotShowAgainCallback)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:

		TSharedPtr<SCheckBox> DoNotShowAgainCheckbox;
	};
}
