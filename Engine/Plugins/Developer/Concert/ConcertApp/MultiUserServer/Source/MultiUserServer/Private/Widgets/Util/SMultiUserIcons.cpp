// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMultiUserIcons.h"

#include "ConcertServerStyle.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.SMultiUserIcons"

void SMultiUserIcons::Construct(const FArguments& InArgs)
{
	const TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	// Please keep this alphabetically sorted if you add new entries
	
	// App
	AddHeader(VerticalBox, LOCTEXT("App", "App"));
	AddCategory(VerticalBox, LOCTEXT("App", "App"), { { TEXT("AppIcon"), TEXT("AppIcon.Small") } });
	
	// New
	AddHeader(VerticalBox, LOCTEXT("Icons", "New Icons"));
	AddCategory(VerticalBox, LOCTEXT("Ack", "Ack"), { TEXT("Concert.Ack.Ack"), TEXT("Concert.Ack.Success"), TEXT("Concert.Ack.Failure") });
	AddCategory(VerticalBox, LOCTEXT("Archive", "Archive"), { TEXT("Concert.Icon.Archive") });
	AddCategory(VerticalBox, LOCTEXT("Client", "Client"), { TEXT("Concert.Icon.Client") });
	AddCategory(VerticalBox, LOCTEXT("CreateMultiUser", "CreateMultiUser"), { TEXT("Concert.Icon.CreateMultiUser") });
	AddCategory(VerticalBox, LOCTEXT("ExportImport", "ExportImport"), { TEXT("Concert.Icon.Export"), TEXT("Concert.Icon.Import") });
	AddCategory(VerticalBox, LOCTEXT("LogServer", "LogServer"), { TEXT("Concert.Icon.LogServer") });
	AddCategory(VerticalBox, LOCTEXT("LogSession", "LogSession"), { TEXT("Concert.Icon.LogSession") });
	AddCategory(VerticalBox, LOCTEXT("MultiUser", "MultiUser"), { TEXT("Concert.Icon.MultiUser") });
	AddCategory(VerticalBox, LOCTEXT("Muting", "Muting"), { TEXT("Concert.Muted"), TEXT("Concert.Unmuted") });
	AddCategory(VerticalBox, LOCTEXT("Package", "Package"), { TEXT("Concert.Icon.Package") });
	AddCategory(VerticalBox, LOCTEXT("PackageChanged", "Package Changed"), { TEXT("Concert.SessionContent.PackageAdded"), TEXT("Concert.SessionContent.PackageDeleted"), TEXT("Concert.SessionContent.PackageRenamed"), TEXT("Concert.SessionContent.PackageSaved") });
	AddCategory(VerticalBox, LOCTEXT("PackageTransmission", "PackageTransmission"), { TEXT("Concert.PackageTransmission.Success"), TEXT("Concert.PackageTransmission.Failure") });
	AddCategory(VerticalBox, LOCTEXT("Server", "Server"), { TEXT("Concert.Icon.Server") });
	
	ChildSlot
	[
		SNew(SBorder)
		.Padding(20.f)
		[
			VerticalBox
		]
	];
}

void SMultiUserIcons::AddHeader(const TSharedRef<SVerticalBox>& VerticalBox, FText Text)
{
	VerticalBox->AddSlot()
	.AutoHeight()
	.Padding(0.f, 10.f, 0.f, 5.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.f, 0.f, 20.f, 0.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(Text)
			.Justification(ETextJustify::Right)
		]
	];
}

void SMultiUserIcons::AddCategory(const TSharedRef<SVerticalBox>& VerticalBox, FText Text, const TArray<FName>& IconNames)
{
	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
	
	VerticalBox->AddSlot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.f, 0.f, 20.f, 0.f)
		.FillWidth(0.3f)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(Text)
			.Justification(ETextJustify::Right)
		]
		
		+SHorizontalBox::Slot()
		.FillWidth(0.7f)
		[
			HorizontalBox
		]
	];
	
	for (const FName& IconName : IconNames)
	{
		HorizontalBox->AddSlot()
		.Padding(5.f)
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FConcertServerStyle::Get().GetBrush(IconName))
		];
	}
}

#undef LOCTEXT_NAMESPACE