// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SVerticalBox;

/** Shows all multi user server icons */
class SMultiUserIcons : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMultiUserIcons)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	void AddHeader(const TSharedRef<SVerticalBox>& VerticalBox, FText Text);
	void AddCategory(const TSharedRef<SVerticalBox>& VerticalBox, FText Text, const TArray<FName>& IconNames);
};
