// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerTabs.h"
#include "UObject/NameTypes.h"

FName ConcertServerTabs::GetSessionBrowserTabId()
{
	static FName SessionBrowserTabId("SessionBrowserTabId");
	return SessionBrowserTabId;
}

FName ConcertServerTabs::GetClientsTabID()
{
	static FName ClientsTabID("ClientsTabID");
	return ClientsTabID;
}
