// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FName;

namespace ConcertServerTabs
{
	FName GetSessionBrowserTabId();
	FName GetClientsTabID();
}
