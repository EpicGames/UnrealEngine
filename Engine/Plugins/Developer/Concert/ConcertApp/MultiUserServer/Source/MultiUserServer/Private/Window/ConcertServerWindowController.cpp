// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerWindowController.h"

#include "ConcertServerEvents.h"
#include "IConcertModule.h"
#include "IConcertServer.h"
#include "IConcertSession.h"
#include "IConcertSyncServer.h"
#include "Widgets/Browser/ConcertServerSessionBrowserController.h"
#include "Widgets/Clients/ConcertClientsTabController.h"
#include "Widgets/SessionTabs/Archived/ArchivedConcertSessionTab.h"
#include "Widgets/SessionTabs/Live/LiveConcertSessionTab.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#define LOCTEXT_NAMESPACE "UnrealMultiUserUI.FConcertServerWindowController"

namespace UE::MultiUserServer
{
	FConcertServerWindowController::FConcertServerWindowController(const FConcertServerWindowInitParams& Params)
		: MultiUserServerLayoutIni(Params.MultiUserServerLayoutIni)
		, ServerInstance(Params.Server)
		, SessionBrowserController(MakeShared<FConcertServerSessionBrowserController>())
		, ClientsController(MakeShared<FConcertClientsTabController>())
		, ConcertComponents(Params.AdditionalConcertComponents)
	{
		ConcertComponents.Add(SessionBrowserController);
		ConcertComponents.Add(ClientsController);

		FCoreDelegates::OnPreExit.AddRaw(this, &FConcertServerWindowController::SaveTabManagerLayout);
	}

	FConcertServerWindowController::~FConcertServerWindowController()
	{
		FCoreDelegates::OnPreExit.RemoveAll(this);
		UnregisterFromSessionDestructionEvents();
	}

	TSharedRef<SWindow> FConcertServerWindowController::CreateWindow()
	{
		FDisplayMetrics DisplayMetrics;
		FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);
		const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayMetrics.PrimaryDisplayWorkAreaRect.Left, DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);

		constexpr bool bEmbedTitleAreaContent = true;
		const FVector2D ClientSize(1000.0f * DPIScaleFactor, 800.0f * DPIScaleFactor);
		TSharedRef<SWindow> RootWindowRef = SNew(SWindow)
			.Title(LOCTEXT("WindowTitle", "Unreal Multi User Server"))
			.CreateTitleBar(!bEmbedTitleAreaContent)
			.SupportsMaximize(true)
			.SupportsMinimize(true)
			.IsInitiallyMaximized(false)
			.IsInitiallyMinimized(false)
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.ClientSize(ClientSize)
			.AdjustInitialSizeAndPositionForDPIScale(false);
		RootWindow = RootWindowRef;
			
		constexpr bool bShowRootWindowImmediately = false;
		FSlateApplication::Get().AddWindow(RootWindowRef, bShowRootWindowImmediately);
		FGlobalTabmanager::Get()->SetRootWindow(RootWindowRef);
		FGlobalTabmanager::Get()->SetAllowWindowMenuBar(true);
		FSlateNotificationManager::Get().SetRootWindow(RootWindowRef);
		
		const TSharedRef<FTabManager::FStack> MainStack = FTabManager::NewStack();
		InitComponents(MainStack);
		const TSharedRef<FTabManager::FArea> MainWindowArea = FTabManager::NewPrimaryArea();
		const TSharedRef<FTabManager::FLayout> DefaultLayout = FTabManager::NewLayout("UnrealMultiUserServerLayout_v1.0");
		MainWindowArea->Split(MainStack);
		DefaultLayout->AddArea(MainWindowArea);
		
		PersistentLayout = FLayoutSaveRestore::LoadFromConfig(MultiUserServerLayoutIni, DefaultLayout);
		TSharedPtr<SWidget> Content = FGlobalTabmanager::Get()->RestoreFrom(PersistentLayout.ToSharedRef(), RootWindow, bEmbedTitleAreaContent, EOutputCanBeNullptr::Never);
		RootWindow->SetContent(Content.ToSharedRef());

		RootWindow->SetOnWindowClosed(FOnWindowClosed::CreateRaw(this, &FConcertServerWindowController::OnWindowClosed));
		RootWindow->ShowWindow();
		constexpr bool bForceWindowToFront = true;
		RootWindow->BringToFront(bForceWindowToFront);

		RegisterForSessionDestructionEvents();
		SessionBrowserController->OpenTab();
		return RootWindowRef;
	}

	void FConcertServerWindowController::OpenSessionTab(const FGuid& SessionId)
	{
		if (const TSharedPtr<FConcertSessionTabBase> SessionTab = GetOrRegisterSessionTab(SessionId))
		{
			SessionTab->OpenSessionTab();
		}
	}

	void FConcertServerWindowController::DestroySessionTab(const FGuid& SessionId)
	{
		// Destructor will handle the rest, e.g. removing the tab from the window
		RegisteredSessions.Remove(SessionId);
	}

	TSharedPtr<FConcertSessionTabBase> FConcertServerWindowController::GetOrRegisterSessionTab(const FGuid& SessionId)
	{
		if (const TSharedRef<FConcertSessionTabBase>* FoundId = RegisteredSessions.Find(SessionId))
		{
			return *FoundId;
		}
		
		if (const TSharedPtr<IConcertServerSession> Session = ServerInstance->GetConcertServer()->GetLiveSession(SessionId))
		{
			const TSharedRef<FLiveConcertSessionTab> SessionTab = MakeShared<FLiveConcertSessionTab>(
				Session.ToSharedRef(),
				ServerInstance,
				RootWindow.ToSharedRef(),
				FLiveConcertSessionTab::FShowConnectedClients::CreateSP(this, &FConcertServerWindowController::ShowConnectedClients)
				);
			RegisteredSessions.Add(SessionId, SessionTab);
			return SessionTab;
		}

		const bool bIsArchivedSession = ServerInstance->GetConcertServer()->GetArchivedSessionInfo(SessionId).IsSet();
		if (bIsArchivedSession)
		{
			const TSharedRef<FArchivedConcertSessionTab> SessionTab = MakeShared<FArchivedConcertSessionTab>(SessionId, ServerInstance, RootWindow.ToSharedRef());
			RegisteredSessions.Add(SessionId, SessionTab);
			return SessionTab;
		}
		
		return nullptr;
	}

	void FConcertServerWindowController::InitComponents(const TSharedRef<FTabManager::FStack>& MainArea)
	{
		const FConcertComponentInitParams Params { ServerInstance, SharedThis(this), MainArea };
		for (const TSharedRef<IConcertComponent>& ConcertComponent : ConcertComponents)
		{
			ConcertComponent->Init(Params);
		}
	}

	void FConcertServerWindowController::RegisterForSessionDestructionEvents()
	{
		ConcertServerEvents::OnLiveSessionDestroyed().AddSP(this, &FConcertServerWindowController::OnLiveSessionDestroyed);
		ConcertServerEvents::OnArchivedSessionDestroyed().AddSP(this, &FConcertServerWindowController::OnArchivedSessionDestroyed);
	}

	void FConcertServerWindowController::UnregisterFromSessionDestructionEvents() const
	{
		ConcertServerEvents::OnLiveSessionDestroyed().RemoveAll(this);
		ConcertServerEvents::OnArchivedSessionDestroyed().RemoveAll(this);
	}

	void FConcertServerWindowController::OnLiveSessionDestroyed(const IConcertServer&, TSharedRef<IConcertServerSession> InLiveSession)
	{
		DestroySessionTab(InLiveSession->GetId());
	}

	void FConcertServerWindowController::OnArchivedSessionDestroyed(const IConcertServer&, const FGuid& InArchivedSessionId)
	{
		DestroySessionTab(InArchivedSessionId);
	}

	void FConcertServerWindowController::ShowConnectedClients(const TSharedRef<IConcertServerSession>& ServerSession)
	{
		ClientsController->ShowConnectedClients(ServerSession->GetId());
	}

	void FConcertServerWindowController::OnWindowClosed(const TSharedRef<SWindow>& Window)
	{
		SaveLayout();
		RootWindow.Reset();
	}

	void FConcertServerWindowController::SaveLayout() const
	{
		if (PersistentLayout)
		{
			FLayoutSaveRestore::SaveToConfig(MultiUserServerLayoutIni, PersistentLayout.ToSharedRef());
		    GConfig->Flush(false, MultiUserServerLayoutIni);
		}
	}

	void FConcertServerWindowController::SaveTabManagerLayout()
	{
		FGlobalTabmanager::Get()->SaveAllVisualState();
	}
}

#undef LOCTEXT_NAMESPACE 
