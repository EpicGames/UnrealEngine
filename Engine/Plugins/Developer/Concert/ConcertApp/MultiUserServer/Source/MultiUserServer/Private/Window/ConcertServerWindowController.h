// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"

class IConcertServer;
class FConcertSessionTabBase;
class FLiveConcertSessionTab;
class IConcertServerSession;
class IConcertSyncServer;
class FOutputLogController;
class SWindow;

namespace UE::MultiUserServer
{
	class IConcertComponent;
	class FConcertServerSessionBrowserController;
	class FConcertClientsTabController;
	
	struct FConcertServerWindowInitParams
	{
		/** The server that the window is supposed to manage */
		TSharedRef<IConcertSyncServer> Server;
		
		/** Config path for server layout ini */
		FString MultiUserServerLayoutIni;
		
		/** Additional modular concert features to enable */
		TArray<TSharedRef<IConcertComponent>> AdditionalConcertComponents;
		
		FConcertServerWindowInitParams(TSharedRef<IConcertSyncServer> Server, FString MultiUserServerLayoutIni = FString())
			: Server(Server)
			, MultiUserServerLayoutIni(MoveTemp(MultiUserServerLayoutIni))
		{}
	};

	/** Responsible for creating the Slate window for the server. Implements controller in the model-view-controller pattern. */
	class FConcertServerWindowController : public TSharedFromThis<FConcertServerWindowController>
	{
	public:
		
		FConcertServerWindowController(const FConcertServerWindowInitParams& Params);
		~FConcertServerWindowController();
		
		TSharedRef<SWindow> CreateWindow();
		TSharedPtr<SWindow> GetRootWindow() const { return RootWindow; }

		/** Opens or draws attention to the tab for the given live or archived session ID */
		void OpenSessionTab(const FGuid& SessionId);

		/** Destroys the tab associated with this live or archived session ID. */
		void DestroySessionTab(const FGuid& SessionId);
		
	private:

		/** The ini file to use for saving the layout */
		FString MultiUserServerLayoutIni;
		/** Holds the current layout for saving later. */
		TSharedPtr<FTabManager::FLayout> PersistentLayout;

		TSharedRef<IConcertSyncServer> ServerInstance;

		/** The main window being managed */
		TSharedPtr<SWindow> RootWindow;
		TMap<FGuid, TSharedRef<FConcertSessionTabBase>> RegisteredSessions;
		
		/** Manages the session browser */
		TSharedRef<FConcertServerSessionBrowserController> SessionBrowserController;
		/** Manages the view of connected clients */
		TSharedRef<FConcertClientsTabController> ClientsController;
		/** Additional external concert components */
		TArray<TSharedRef<IConcertComponent>> ConcertComponents;
		
		void InitComponents(const TSharedRef<FTabManager::FStack>& MainArea);

		/** Gets the manager for a session tab if the session ID is valid */
		TSharedPtr<FConcertSessionTabBase> GetOrRegisterSessionTab(const FGuid& SessionId);

		void RegisterForSessionDestructionEvents();
		void UnregisterFromSessionDestructionEvents() const;
		void OnLiveSessionDestroyed(const IConcertServer&, TSharedRef<IConcertServerSession> InLiveSession);
		void OnArchivedSessionDestroyed(const IConcertServer&, const FGuid& InArchivedSessionId);

		void ShowConnectedClients(const TSharedRef<IConcertServerSession>& ServerSession);
		
		void OnWindowClosed(const TSharedRef<SWindow>& Window);
		void SaveLayout() const;

		/** Saves the layout of the global tab manager (on shutdown). */
		void SaveTabManagerLayout();
	};
}
