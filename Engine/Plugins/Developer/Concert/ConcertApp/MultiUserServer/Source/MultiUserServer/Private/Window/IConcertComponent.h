// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"

class IConcertSyncServer;

namespace UE::MultiUserServer
{
	class FConcertServerWindowController;

	struct FConcertComponentInitParams
	{
		/** The server being managed */
		TSharedRef<IConcertSyncServer> Server;

		/** Manages the server slate application */
		TSharedRef<FConcertServerWindowController> WindowController;

		/** The root area of the main window layout. Add tabs to this. */
		TSharedRef<FTabManager::FStack> MainStack;

		FConcertComponentInitParams(TSharedRef<IConcertSyncServer> Server, TSharedRef<FConcertServerWindowController> WindowController, TSharedRef<FTabManager::FStack> MainStack)
			: Server(MoveTemp(Server))
			, WindowController(MoveTemp(WindowController))
			, MainStack(MoveTemp(MainStack))
		{}
	};

	/** Provides the base interface for elements in the concert server UI. */
	class IConcertComponent
	{
	public:

		virtual ~IConcertComponent() = default;

		/** Initialises the component, e.g. registering tab spawners, etc. */
		virtual void Init(const FConcertComponentInitParams& Params) {}
	};
}
