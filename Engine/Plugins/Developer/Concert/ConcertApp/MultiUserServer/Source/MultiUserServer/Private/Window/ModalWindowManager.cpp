// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModalWindowManager.h"

#include "Dialog/SCustomDialog.h"

UE::MultiUserServer::FModalWindowManager::FModalWindowManager(const TSharedRef<SWindow>& MainWindow)
{
	MainWindow->GetOnWindowActivatedEvent().AddRaw(this, &FModalWindowManager::OnMainWindowActivated);

	// Fake modal windows should be closed when the root window is closed
	MainWindow->GetOnWindowClosedEvent().AddLambda([this](const TSharedRef<SWindow>&)
	{
		// One way to close root window while fake modal is open:
			// 1. Hover MU server window icon (on Windows)
			// 2. Close root window
		bRootWindowIsValid = false;
		
		for (TWeakPtr<SWindow> ModalWindow : TArray(WindowStack))
		{
			if (TSharedPtr<SWindow> Pin = ModalWindow.Pin())
			{
				Pin->RequestDestroyWindow();
			}
		}
		WindowStack.Empty();
	});
}

void UE::MultiUserServer::FModalWindowManager::ShowFakeModalWindow(const TSharedRef<SCustomDialog>& Dialog)
{
	if (ensureMsgf(bRootWindowIsValid, TEXT("Root window has already been destroyed!")))
	{
		Dialog->Show();
		PushWindow(Dialog);
	}
}

void UE::MultiUserServer::FModalWindowManager::PushWindow(const TSharedRef<SWindow>& Window)
{
	if (ensureMsgf(bRootWindowIsValid, TEXT("Root window has already been destroyed!")))
	{
		check(!WindowStack.Contains(Window));
		WindowStack.Push(Window);
	
		Window->GetOnWindowClosedEvent().AddLambda([this](const TSharedRef<SWindow>& Window)
		{
			// Might request window destruction twice
			if (WindowStack.Contains(Window))
			{
				PopWindow();
			}
		});
	}
}

void UE::MultiUserServer::FModalWindowManager::PopWindow()
{
	WindowStack.Pop();
}

void UE::MultiUserServer::FModalWindowManager::OnMainWindowActivated()
{
	if (WindowStack.Num() > 0)
	{
		if (const TSharedPtr<SWindow> PinnedWindow = WindowStack.Top().Pin())
		{
			PinnedWindow->BringToFront();
			PinnedWindow->FlashWindow();
		}
	}
}
