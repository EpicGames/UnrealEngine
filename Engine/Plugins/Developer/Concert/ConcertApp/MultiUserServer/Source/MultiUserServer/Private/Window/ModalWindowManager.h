// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWindow;
class SCustomDialog;

namespace UE::MultiUserServer
{
	/**
	 * Manages "fake" modal windows because the server cannot use "true" modal windows.
	 *
	 * True modal windows block the main thread which is not an option because the server must continue responding to
	 * networked requests.
	 */
	class FModalWindowManager
	{
	public:

		FModalWindowManager(const TSharedRef<SWindow>& MainWindow);

		void ShowFakeModalWindow(const TSharedRef<SCustomDialog>& Dialog);

		/** Called when a new fake modal window should be tracked on top of all windows. */
		void PushWindow(const TSharedRef<SWindow>& Window);
		/** Called when a fake modal window should no longer be tracked, i.e. when it is closed. */
		void PopWindow();

	private:

		TArray<TWeakPtr<SWindow>> WindowStack;

		/** Modal windows can only be added when this is true */
		bool bRootWindowIsValid = true;

		void OnMainWindowActivated();
	};
}


